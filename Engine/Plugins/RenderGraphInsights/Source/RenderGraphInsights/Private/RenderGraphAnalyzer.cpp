// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphAnalyzer.h"

#include "RenderGraphProvider.h"

#define LOCTEXT_NAMESPACE "RenderGraphProvider"

namespace UE
{
namespace RenderGraphInsights
{

FRenderGraphAnalyzer::FRenderGraphAnalyzer(TraceServices::IAnalysisSession& InSession, FRenderGraphProvider& InRenderGraphProvider)
	: Session(InSession)
	, Provider(InRenderGraphProvider)
	, bIsValidGraph(false)
{}

void FRenderGraphAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	UE::Trace::IAnalyzer::FInterfaceBuilder& Builder = Context.InterfaceBuilder;

	Builder.RouteEvent(RouteId_Graph, "RDGTrace", "GraphMessage");
	Builder.RouteEvent(RouteId_GraphEnd, "RDGTrace", "GraphEndMessage");
	Builder.RouteEvent(RouteId_Scope, "RDGTrace", "ScopeMessage");
	Builder.RouteEvent(RouteId_Pass, "RDGTrace", "PassMessage");
	Builder.RouteEvent(RouteId_Texture, "RDGTrace", "TextureMessage");
	Builder.RouteEvent(RouteId_Buffer, "RDGTrace", "BufferMessage");

	bIsValidGraph = false;
}

bool FRenderGraphAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FRenderGraphAnalyzer"));

	TraceServices::FAnalysisSessionEditScope _(Session);
	switch (RouteId)
	{
	case RouteId_Graph:
	{
		ensure(!bIsValidGraph);
		double EndTime{};
		Provider.AddGraph(Context, EndTime);
		Session.UpdateDurationSeconds(EndTime);
		bIsValidGraph = true;
		break;
	}
	case RouteId_GraphEnd:
		if (bIsValidGraph)
		{
			Provider.AddGraphEnd();
			bIsValidGraph = false;
		}
		break;
	case RouteId_Scope:
		if (bIsValidGraph)
		{
			Provider.AddScope(FScopePacket(Context));
		}
		break;
	case RouteId_Pass:
		if (bIsValidGraph)
		{
			Provider.AddPass(FPassPacket(Context));
		}
		break;
	case RouteId_Texture:
		if (bIsValidGraph)
		{
			Provider.AddTexture(FTexturePacket(Context));
		}
		break;
	case RouteId_Buffer:
		if (bIsValidGraph)
		{
			Provider.AddBuffer(FBufferPacket(Context));
		}
		break;
	}
	return true;
}

} //namespace RenderGraphInsights
} //namespace UE

#undef LOCTEXT_NAMESPACE
