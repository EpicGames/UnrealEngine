// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/Analyzer.h"

namespace TraceServices { class IAnalysisSession; }

namespace UE
{
namespace RenderGraphInsights
{ 

class FRenderGraphProvider;

class FRenderGraphAnalyzer : public UE::Trace::IAnalyzer
{
public:
	FRenderGraphAnalyzer(TraceServices::IAnalysisSession& InSession, FRenderGraphProvider& InRenderGraphProvider);

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override {}
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

private:
	enum : uint16
	{
		RouteId_Graph,
		RouteId_GraphEnd,
		RouteId_Scope,
		RouteId_Pass,
		RouteId_Buffer,
		RouteId_Texture
	};

	TraceServices::IAnalysisSession& Session;
	FRenderGraphProvider& Provider;
	bool bIsValidGraph;
};

} //namespace RenderGraphInsights
} //namespace UE
