// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphInsightsModule.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"


namespace UE
{
namespace RenderGraphInsights
{

FRenderGraphInsightsModule& FRenderGraphInsightsModule::Get()
{
	return FModuleManager::LoadModuleChecked<FRenderGraphInsightsModule>("RenderGraphInsights");
}

void FRenderGraphInsightsModule::StartupModule()
{
	IModularFeatures::Get().RegisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
	IModularFeatures::Get().RegisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
}

void FRenderGraphInsightsModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(TraceServices::ModuleFeatureName, &TraceModule);
	IModularFeatures::Get().UnregisterModularFeature(UE::Insights::Timing::TimingViewExtenderFeatureName, &TimingViewExtender);
}

} //namespace SlateInsights
} //namespace UE

IMPLEMENT_MODULE(UE::RenderGraphInsights::FRenderGraphInsightsModule, RenderGraphInsights);
