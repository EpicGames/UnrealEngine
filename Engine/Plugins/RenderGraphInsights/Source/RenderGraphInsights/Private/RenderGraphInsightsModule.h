// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

#include "RenderGraphTraceModule.h"
#include "RenderGraphTimingViewExtender.h"

class FTabManager;

namespace UE
{
namespace RenderGraphInsights
{

class FRenderGraphInsightsModule : public IModuleInterface
{
public:
	static FRenderGraphInsightsModule& Get();

	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface interface

private:
	FRenderGraphTraceModule TraceModule;
	FRenderGraphTimingViewExtender TimingViewExtender;
	TWeakPtr<FTabManager> InsightsTabManager;
};

} //namespace RenderGraphInsights
} //namespace UE
