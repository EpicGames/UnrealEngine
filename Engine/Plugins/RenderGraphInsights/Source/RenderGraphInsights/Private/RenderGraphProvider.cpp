// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphProvider.h"

namespace UE
{
namespace RenderGraphInsights
{

INSIGHTS_IMPLEMENT_RTTI(FPacket)
INSIGHTS_IMPLEMENT_RTTI(FScopePacket)
INSIGHTS_IMPLEMENT_RTTI(FResourcePacket)
INSIGHTS_IMPLEMENT_RTTI(FTexturePacket)
INSIGHTS_IMPLEMENT_RTTI(FBufferPacket)
INSIGHTS_IMPLEMENT_RTTI(FPassPacket)
INSIGHTS_IMPLEMENT_RTTI(FGraphPacket)
INSIGHTS_IMPLEMENT_RTTI(FPassIntervalPacket)

FName FRenderGraphProvider::ProviderName("RenderGraphProvider");

template <typename ObjectType>
TRDGHandle<ObjectType, uint32> ConvertTo32Bits(TRDGHandle<ObjectType, uint16> Handle)
{
	if (Handle.IsValid())
	{
		return TRDGHandle<ObjectType, uint32>(Handle.GetIndex());
	}
	return {};
}

template <typename HandleType>
HandleType SerializeHandle(const UE::Trace::IAnalyzer::FEventData& EventData, const ANSICHAR* FieldName)
{
	if (EventData.GetValue<bool>("IsHandleType32Bits"))
	{
		return EventData.GetValue<HandleType>(FieldName);
	}
	return ConvertTo32Bits(EventData.GetValue<TRDGHandle<typename HandleType::ObjectType, uint16>>(FieldName));
}

template <typename HandleType>
TArray<HandleType> SerializeHandleArray(const UE::Trace::IAnalyzer::FEventData& EventData, const ANSICHAR* FieldName)
{
	if (EventData.GetValue<bool>("IsHandleType32Bits"))
	{
		static_assert(sizeof(typename HandleType::IndexType) == 4);
		return TArray<HandleType>(EventData.GetArrayView<HandleType>(FieldName));
	}

	using HandleType16 = TRDGHandle<typename HandleType::ObjectType, uint16>;
	TConstArrayView<HandleType16> Handles = EventData.GetArrayView<HandleType16>(FieldName);
	TArray<HandleType> Result;
	Result.Reserve(Handles.Num());
	for (HandleType16 Handle : Handles)
	{
		Result.Emplace(ConvertTo32Bits(Handle));
	}
	return Result;
}

FString GetSizeName(uint32 Bytes)
{
	const uint32 KB = 1024;
	const uint32 MB = 1024 * 1024;

	if (Bytes < MB)
	{
		return FString::Printf(TEXT(" (%.3fKB)"), (float)Bytes / (float)KB);
	}
	else
	{
		return FString::Printf(TEXT(" (%.3fMB)"), (float)Bytes / (float)MB);
	}
}

FStringView GetString(const UE::Trace::IAnalyzer::FEventData& EventData, const ANSICHAR* FieldName)
{
	FStringView Value;
	EventData.GetString(FieldName, Value);
	return Value;
}

FPacket::FPacket(const UE::Trace::IAnalyzer::FOnEventContext& Context)
	: Name(GetString(Context.EventData, "Name"))
	, StartTime(Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>("StartCycles")))
	, EndTime(Context.EventTime.AsSeconds(Context.EventData.GetValue<uint64>("EndCycles")))
{}

FPassIntervalPacket::FPassIntervalPacket(const UE::Trace::IAnalyzer::FOnEventContext& Context)
	: FPacket(Context)
	, FirstPass(SerializeHandle<FRDGPassHandle>(Context.EventData, "FirstPass"))
	, LastPass(SerializeHandle<FRDGPassHandle>(Context.EventData, "LastPass"))
{}

FScopePacket::FScopePacket(const UE::Trace::IAnalyzer::FOnEventContext& Context)
	: FPassIntervalPacket(Context)
	, Depth(Context.EventData.GetValue<uint16>("Depth"))
{}

FResourcePacket::FResourcePacket(const UE::Trace::IAnalyzer::FOnEventContext& Context)
	: FPassIntervalPacket(Context)
	, Order(Context.EventData.GetValue<uint16>("Order"))
	, SizeInBytes(Context.EventData.GetValue<uint64>("SizeInBytes"))
	, Passes(SerializeHandleArray<FRDGPassHandle>(Context.EventData, "Passes"))
	, bExternal(Context.EventData.GetValue<bool>("IsExternal"))
	, bExtracted(Context.EventData.GetValue<bool>("IsExtracted"))
	, bCulled(Context.EventData.GetValue<bool>("IsCulled"))
	, bTrackingSkipped(Context.EventData.GetValue<bool>("IsTrackingSkipped"))
	, bTransient(Context.EventData.GetValue<bool>("IsTransient"))
	, bTransientUntracked(Context.EventData.GetValue<bool>("IsTransientUntracked"))
	, bTransientCacheHit(Context.EventData.GetValue<bool>("IsTransientCacheHit"))
{
	const auto TransientAllocationOffsetMins   = Context.EventData.GetArrayView<uint64>("TransientAllocationOffsetMins");
	const auto TransientAllocationOffsetMaxs   = Context.EventData.GetArrayView<uint64>("TransientAllocationOffsetMaxs");
	const auto TransientAllocationMemoryRanges = Context.EventData.GetArrayView<uint16>("TransientAllocationMemoryRanges");
	check(TransientAllocationOffsetMins.Num() == TransientAllocationOffsetMaxs.Num() && TransientAllocationOffsetMaxs.Num() == TransientAllocationMemoryRanges.Num());

	TransientAllocations.SetNum(TransientAllocationOffsetMins.Num());

	for (int32 LocalIndex = 0; LocalIndex < TransientAllocationOffsetMins.Num(); ++LocalIndex)
	{
		TransientAllocations[LocalIndex].OffsetMin = TransientAllocationOffsetMins[LocalIndex];
		TransientAllocations[LocalIndex].OffsetMax = TransientAllocationOffsetMaxs[LocalIndex];
		TransientAllocations[LocalIndex].MemoryRangeIndex = TransientAllocationMemoryRanges[LocalIndex];
	}

	TransientAcquirePass = Context.EventData.GetValue<FRDGPassHandle>("TransientAcquirePass");
	TransientDiscardPass = Context.EventData.GetValue<FRDGPassHandle>("TransientDiscardPass");

	if (Passes.Num())
	{
		FirstPass = Passes[0];
		LastPass  = Passes.Last();
	}
}

FTexturePacket::FTexturePacket(const UE::Trace::IAnalyzer::FOnEventContext& Context)
	: FResourcePacket(Context)
	, Handle(SerializeHandle<FRDGTextureHandle>(Context.EventData, "Handle"))
	, NextOwnerHandle(SerializeHandle<FRDGTextureHandle>(Context.EventData, "NextOwnerHandle"))
{
	Desc.Flags = ETextureCreateFlags(Context.EventData.GetValue<uint64>("CreateFlags"));
	Desc.Dimension = ETextureDimension(Context.EventData.GetValue<uint16>("Dimension"));
	Desc.Format = EPixelFormat(Context.EventData.GetValue<uint16>("Format"));
	Desc.Extent.X = Context.EventData.GetValue<uint32>("ExtentX");
	Desc.Extent.Y = Context.EventData.GetValue<uint32>("ExtentY");
	Desc.Depth = Context.EventData.GetValue<uint16>("Depth");
	Desc.ArraySize = Context.EventData.GetValue<uint16>("ArraySize");
	Desc.NumMips = Context.EventData.GetValue<uint8>("NumMips");
	Desc.NumSamples = Context.EventData.GetValue<uint8>("NumSamples");

	Name += GetSizeName(SizeInBytes);
}

FBufferPacket::FBufferPacket(const UE::Trace::IAnalyzer::FOnEventContext& Context)
	: FResourcePacket(Context)
	, Handle(SerializeHandle<FRDGBufferHandle>(Context.EventData, "Handle"))
	, NextOwnerHandle(SerializeHandle<FRDGBufferHandle>(Context.EventData, "NextOwnerHandle"))
{
	Desc.Usage = EBufferUsageFlags(Context.EventData.GetValue<uint32>("UsageFlags"));
	Desc.BytesPerElement = Context.EventData.GetValue<uint32>("BytesPerElement");
	Desc.NumElements = Context.EventData.GetValue<uint32>("NumElements");

	SizeInBytes = Desc.BytesPerElement * Desc.NumElements;
	Name += GetSizeName(SizeInBytes);
}

FPassPacket::FPassPacket(const UE::Trace::IAnalyzer::FOnEventContext& Context)
	: FPacket(Context)
	, Textures(SerializeHandleArray<FRDGTextureHandle>(Context.EventData, "Textures"))
	, Buffers(SerializeHandleArray<FRDGBufferHandle>(Context.EventData, "Buffers"))
	, Handle(SerializeHandle<FRDGPassHandle>(Context.EventData, "Handle"))
	, GraphicsForkPass(SerializeHandle<FRDGPassHandle>(Context.EventData, "GraphicsForkPass"))
	, GraphicsJoinPass(SerializeHandle<FRDGPassHandle>(Context.EventData, "GraphicsJoinPass"))
	, Flags(ERDGPassFlags(Context.EventData.GetValue<uint16>("Flags")))
	, Pipeline(ERHIPipeline(Context.EventData.GetValue<uint8>("Pipeline")))
	, bCulled(Context.EventData.GetValue<bool>("IsCulled"))
	, bAsyncComputeBegin(Context.EventData.GetValue<bool>("IsAsyncComputeBegin"))
	, bAsyncComputeEnd(Context.EventData.GetValue<bool>("IsAsyncComputeEnd"))
	, bSkipRenderPassBegin(Context.EventData.GetValue<bool>("SkipRenderPassBegin"))
	, bSkipRenderPassEnd(Context.EventData.GetValue<bool>("SkipRenderPassEnd"))
	, bParallelExecuteBegin(Context.EventData.GetValue<bool>("IsParallelExecuteBegin"))
	, bParallelExecuteEnd(Context.EventData.GetValue<bool>("IsParallelExecuteEnd"))
	, bParallelExecute(Context.EventData.GetValue<bool>("IsParallelExecute"))
	, bParallelExecuteAllowed(Context.EventData.GetValue<bool>("IsParallelExecuteAllowed"))
	, bParallelExecuteAsyncAllowed(Context.EventData.GetValue<bool>("IsParallelExecuteAsyncAllowed"))
{}

static const uint64 PageSize = 1024;

FGraphPacket::FGraphPacket(TraceServices::ILinearAllocator& Allocator, const UE::Trace::IAnalyzer::FOnEventContext& Context)
	: FPacket(Context)
	, Scopes(Allocator, PageSize)
	, Passes(Allocator, PageSize)
	, Textures(Allocator, PageSize)
	, Buffers(Allocator, PageSize)
	, PassCount(Context.EventData.GetValue<uint16>("PassCount"))
{
	NormalizedPassDuration = (EndTime - StartTime) / double(PassCount);

	const auto CommitSizes = Context.EventData.GetArrayView<uint64>("TransientMemoryCommitSizes");
	const auto Capacities  = Context.EventData.GetArrayView<uint64>("TransientMemoryCapacities");
	const auto Flags       = Context.EventData.GetArrayView<uint8>("TransientMemoryFlags");
	check(CommitSizes.Num() == Capacities.Num() && Capacities.Num() == Flags.Num());

	uint64 CurrentOffset = 0;
	TransientMemoryRangeByteOffsets.Reserve(CommitSizes.Num());

	for (int32 Index = 0; Index < CommitSizes.Num(); ++Index)
	{
		auto& MemoryRange = TransientAllocationStats.MemoryRanges.AddDefaulted_GetRef();
		MemoryRange.CommitSize = CommitSizes[Index];
		MemoryRange.Capacity = Capacities[Index];
		MemoryRange.Flags = (FRHITransientAllocationStats::EMemoryRangeFlags)Flags[Index];

		TransientMemoryRangeByteOffsets.Emplace(CurrentOffset);
		CurrentOffset += MemoryRange.CommitSize;
	}
}

FRenderGraphProvider::FRenderGraphProvider(TraceServices::IAnalysisSession& InSession)
	: Session(InSession)
	, GraphTimeline(Session.GetLinearAllocator())
{}

void FRenderGraphProvider::AddGraph(const UE::Trace::IAnalyzer::FOnEventContext& Context, double& OutEndTime)
{
	CurrentGraph = MakeShared<FGraphPacket>(Session.GetLinearAllocator(), Context);
	SanitizeName(CurrentGraph->Name);

	OutEndTime = CurrentGraph->EndTime;
}

void FRenderGraphProvider::AddGraphEnd()
{
	const double EndTime = CurrentGraph->EndTime;
	const uint64 EventId = GraphTimeline.EmplaceBeginEvent(CurrentGraph->StartTime, MoveTemp(CurrentGraph));
	GraphTimeline.EndEvent(EventId, EndTime);
}

void FRenderGraphProvider::AddScope(FScopePacket InScope)
{
	const FPassPacket& FirstPass = *CurrentGraph->GetPass(InScope.FirstPass);
	const FPassPacket& LastPass  = *CurrentGraph->GetPass(InScope.LastPass);

	InScope.Graph = CurrentGraph.Get();
	InScope.StartTime = FirstPass.StartTime;
	InScope.EndTime = LastPass.EndTime;

	CurrentGraph->ScopeDepth = FMath::Max(CurrentGraph->ScopeDepth, InScope.Depth);
	CurrentGraph->Scopes.EmplaceBack(InScope);
}

void FRenderGraphProvider::AddPass(FPassPacket InPass)
{
	InPass.Graph = CurrentGraph.Get();

	const double NormalizedPassDuration = CurrentGraph->NormalizedPassDuration;
	const uint32 PassIndex = CurrentGraph->Passes.Num();
	InPass.StartTime = CurrentGraph->StartTime + NormalizedPassDuration * double(PassIndex);
	InPass.EndTime = InPass.StartTime + NormalizedPassDuration;

	CurrentGraph->Passes.EmplaceBack(InPass);
}

void FRenderGraphProvider::SetupResource(FResourcePacket& Resource)
{
	Resource.Graph = CurrentGraph.Get();

	if (Resource.bCulled)
	{
		return;
	}

	const FPassPacket* FirstPass = CurrentGraph->GetPass(Resource.FirstPass);
	const FPassPacket* LastPass  = CurrentGraph->GetPass(Resource.LastPass);

	if (Resource.bExternal || Resource.bTransientUntracked)
	{
		FirstPass = CurrentGraph->GetProloguePass();
	}

	if (Resource.bExtracted || Resource.bTransientUntracked)
	{
		LastPass = CurrentGraph->GetEpiloguePass();
	}

	Resource.StartTime = FirstPass->StartTime;
	Resource.EndTime = LastPass->EndTime;
}

void FRenderGraphProvider::AddTexture(const FTexturePacket InTexture)
{
	FTexturePacket& Texture = CurrentGraph->Textures.EmplaceBack(InTexture);
	Texture.Index = CurrentGraph->Textures.Num() - 1;
	SetupResource(Texture);

	if (const FTexturePacket* const* PreviousOwnerPtr = CurrentGraph->TextureHandleToPreviousOwner.Find(Texture.Handle))
	{
		const FTexturePacket* PreviousOwner = *PreviousOwnerPtr;
		Texture.PrevousOwnerHandle = PreviousOwner->Handle;
	}

	if (Texture.NextOwnerHandle.IsValid())
	{
		CurrentGraph->TextureHandleToPreviousOwner.Emplace(Texture.NextOwnerHandle, &Texture);
	}
}

void FRenderGraphProvider::AddBuffer(const FBufferPacket InBuffer)
{
	FBufferPacket& Buffer = CurrentGraph->Buffers.EmplaceBack(InBuffer);
	Buffer.Index = CurrentGraph->Buffers.Num() - 1;
	SetupResource(Buffer);

	if (const FBufferPacket* const* PreviousOwnerPtr = CurrentGraph->BufferHandleToPreviousOwner.Find(Buffer.Handle))
	{
		const FBufferPacket* PreviousOwner = *PreviousOwnerPtr;
		Buffer.PrevousOwnerHandle = PreviousOwner->Handle;
	}

	if (Buffer.NextOwnerHandle.IsValid())
	{
		CurrentGraph->BufferHandleToPreviousOwner.Emplace(Buffer.NextOwnerHandle, &Buffer);
	}
}

} //namespace RenderGraphInsights
} //namespace UE