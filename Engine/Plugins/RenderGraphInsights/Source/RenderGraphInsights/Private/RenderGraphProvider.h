// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InsightsCore/Common/SimpleRtti.h"
#include "Model/IntervalTimeline.h"
#include "RenderGraphResources.h"
#include "RHITransientResourceAllocator.h"
#include "Trace/Analyzer.h"
#include "TraceServices/Model/AnalysisSession.h"

enum class ERHIPipeline : uint8;

namespace TraceServices { class IAnalysisSession; }

namespace UE
{
namespace RenderGraphInsights
{

inline void SanitizeName(FString& Name)
{
	if (Name.IsEmpty())
	{
		Name = TEXT("<unnamed>");
	}
}

inline FString GetSanitizedName(FString Name)
{
	SanitizeName(Name);
	return MoveTemp(Name);
}

class FGraphPacket;

const uint32 kInvalidVisibleIndex = ~0u;

class FPacket
{
	INSIGHTS_DECLARE_RTTI_BASE(FPacket);
public:
	virtual ~FPacket() = default;

	const FGraphPacket* Graph{};
	FString Name;
	double StartTime{};
	double EndTime{};

	FPacket(const UE::Trace::IAnalyzer::FOnEventContext& Context);
};

class FPassIntervalPacket : public FPacket
{
	INSIGHTS_DECLARE_RTTI(FPassIntervalPacket, FPacket)
public:
	FRDGPassHandle FirstPass;
	FRDGPassHandle LastPass;

	FPassIntervalPacket(const UE::Trace::IAnalyzer::FOnEventContext& Context);
};

class FScopePacket : public FPassIntervalPacket
{
	INSIGHTS_DECLARE_RTTI(FScopePacket, FPassIntervalPacket)
public:
	uint32 Depth{};

	mutable uint32 VisibleIndex = kInvalidVisibleIndex;

	FScopePacket(const UE::Trace::IAnalyzer::FOnEventContext& Context);
};

class FResourcePacket : public FPassIntervalPacket
{
	INSIGHTS_DECLARE_RTTI(FResourcePacket, FPassIntervalPacket)
public:
	uint32 Index{};
	uint32 Order{};
	uint64 SizeInBytes{};
	FRHITransientAllocationStats::FAllocationArray TransientAllocations;

	TArray<FRDGPassHandle> Passes;

	mutable TArray<uint32, TInlineAllocator<1>> VisibleItems;

	FRDGPassHandle TransientAcquirePass;
	FRDGPassHandle TransientDiscardPass;

	bool bExternal{};
	bool bExtracted{};
	bool bCulled{};
	bool bTrackingSkipped{};
	bool bTransient{};
	bool bTransientUntracked{};
	bool bTransientCacheHit{};

	FResourcePacket(const UE::Trace::IAnalyzer::FOnEventContext& Context);
};

class FTexturePacket : public FResourcePacket
{
	INSIGHTS_DECLARE_RTTI(FTexturePacket, FResourcePacket)
public:
	FRDGTextureHandle Handle;
	FRDGTextureHandle NextOwnerHandle;
	FRDGTextureHandle PrevousOwnerHandle;
	FRDGTextureDesc Desc;

	FTexturePacket(const UE::Trace::IAnalyzer::FOnEventContext& Context);
};

class FBufferPacket : public FResourcePacket
{
	INSIGHTS_DECLARE_RTTI(FBufferPacket, FResourcePacket)
public:
	FRDGBufferHandle Handle;
	FRDGBufferHandle NextOwnerHandle;
	FRDGBufferHandle PrevousOwnerHandle;
	FRDGBufferDesc Desc;

	FBufferPacket(const UE::Trace::IAnalyzer::FOnEventContext& Context);
};

class FPassPacket : public FPacket
{
	INSIGHTS_DECLARE_RTTI(FPassPacket, FPacket)
public:
	TArray<FRDGTextureHandle> Textures;
	TArray<FRDGBufferHandle> Buffers;

	mutable uint32 VisibleIndex = kInvalidVisibleIndex;

	FRDGPassHandle Handle;
	FRDGPassHandle GraphicsForkPass;
	FRDGPassHandle GraphicsJoinPass;
	ERDGPassFlags Flags{};
	ERHIPipeline Pipeline{};
	bool bCulled{};
	bool bAsyncComputeBegin{};
	bool bAsyncComputeEnd{};
	bool bSkipRenderPassBegin{};
	bool bSkipRenderPassEnd{};
	bool bParallelExecuteBegin{};
	bool bParallelExecuteEnd{};
	bool bParallelExecute{};
	bool bParallelExecuteAllowed{};
	bool bParallelExecuteAsyncAllowed{};

	FPassPacket(const UE::Trace::IAnalyzer::FOnEventContext& Context);
};

class FGraphPacket : public FPacket
{
	INSIGHTS_DECLARE_RTTI(FGraphPacket, FPacket)
public:
	double NormalizedPassDuration{};

	TraceServices::TPagedArray<FScopePacket> Scopes;
	TraceServices::TPagedArray<FPassPacket> Passes;
	TraceServices::TPagedArray<FTexturePacket> Textures;
	TraceServices::TPagedArray<FBufferPacket> Buffers;

	TMap<FRDGBufferHandle, const FBufferPacket*> BufferHandleToPreviousOwner;
	TMap<FRDGTextureHandle, const FTexturePacket*> TextureHandleToPreviousOwner;

	mutable uint32 VisibleIndex = kInvalidVisibleIndex;

	uint32 ScopeDepth{};
	uint32 PassCount{};

	FRHITransientAllocationStats TransientAllocationStats;
	TArray<uint64> TransientMemoryRangeByteOffsets;

	FGraphPacket(TraceServices::ILinearAllocator& Allocator, const UE::Trace::IAnalyzer::FOnEventContext& Context);

	const FPassPacket* GetProloguePass() const
	{
		return &Passes[0];
	}

	const FPassPacket* GetEpiloguePass() const
	{
		return &Passes.Last();
	}

	const FPassPacket* GetPass(FRDGPassHandle Handle) const
	{
		return &Passes[Handle.GetIndex()];
	}

	const FTexturePacket* GetTexture(FRDGTextureHandle Handle) const
	{
		return &Textures[Handle.GetIndex()];
	}

	const FBufferPacket* GetBuffer(FRDGBufferHandle Handle) const
	{
		return &Buffers[Handle.GetIndex()];
	}
};

class FRenderGraphProvider : public TraceServices::IProvider
{
public:
	static FName ProviderName;

	FRenderGraphProvider(TraceServices::IAnalysisSession& InSession);

	void AddGraph(const UE::Trace::IAnalyzer::FOnEventContext& Context, double& OutEndTime);
	void AddGraphEnd();
	void AddScope(FScopePacket Scope);
	void AddPass(FPassPacket Pass);
	void AddTexture(FTexturePacket Texture);
	void AddBuffer(FBufferPacket Buffer);

	using TGraphTimeline = TraceServices::TIntervalTimeline<TSharedPtr<FGraphPacket>>;
	const TGraphTimeline& GetGraphTimeline() const
	{
		Session.ReadAccessCheck();
		return GraphTimeline;
	}

private:
	void SetupResource(FResourcePacket& ResourcePacket);

	TraceServices::IAnalysisSession& Session;

	TSharedPtr<FGraphPacket> CurrentGraph;

	TGraphTimeline GraphTimeline;
};

inline bool Intersects(const FPassIntervalPacket& A, const FPassPacket& B)
{
	return A.FirstPass <= B.Handle && A.LastPass >= B.Handle;
}

inline bool Intersects(const FPassIntervalPacket& A, const FPassIntervalPacket& B)
{
	return !(B.LastPass < A.FirstPass || A.LastPass < B.FirstPass);
}

} //namespace RenderGraphInsights
} //namespace UE
