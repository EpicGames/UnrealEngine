// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphTimingViewExtender.h"
#include "Insights/ITimingViewSession.h"
#include "Insights/IUnrealInsightsModule.h"

#define LOCTEXT_NAMESPACE "RenderGraphTimingViewExtender"

namespace UE
{
namespace RenderGraphInsights
{

void FRenderGraphTimingViewExtender::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData == nullptr)
	{
		PerSessionData = &PerSessionDataMap.Add(&InSession);
		PerSessionData->SharedData = MakeUnique<FRenderGraphTimingViewSession>();

		PerSessionData->SharedData->OnBeginSession(InSession);
	}
	else
	{
		PerSessionData->SharedData->OnBeginSession(InSession);
	}
}

void FRenderGraphTimingViewExtender::OnEndSession(UE::Insights::Timing::ITimingViewSession& InSession)
{
	if (InSession.GetName() != FInsightsManagerTabs::TimingProfilerTabId)
	{
		return;
	}

	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData != nullptr)
	{
		PerSessionData->SharedData->OnEndSession(InSession);
	}

	PerSessionDataMap.Remove(&InSession);
}

void FRenderGraphTimingViewExtender::Tick(UE::Insights::Timing::ITimingViewSession& InSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if(PerSessionData != nullptr)
	{
		PerSessionData->SharedData->Tick(InSession, InAnalysisSession);
	}
}

void FRenderGraphTimingViewExtender::ExtendFilterMenu(UE::Insights::Timing::ITimingViewSession& InSession, FMenuBuilder& InMenuBuilder)
{
	FPerSessionData* PerSessionData = PerSessionDataMap.Find(&InSession);
	if (PerSessionData != nullptr)
	{
		PerSessionData->SharedData->ExtendFilterMenu(InMenuBuilder);
	}
}

} //namespace RenderGraphInsights
} //namespace UE

#undef LOCTEXT_NAMESPACE
