// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphTimingViewSession.h"

#include "Insights/ITimingViewSession.h"

#include "RenderGraphTrack.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

#define LOCTEXT_NAMESPACE "RenderGraphTimingViewSession"

namespace UE
{
namespace RenderGraphInsights
{

void FRenderGraphTimingViewSession::OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	TimingViewSession = &InTimingViewSession;

	Track.Reset();
}

void FRenderGraphTimingViewSession::OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession)
{
	Track.Reset();
	TimingViewSession = nullptr;
}

void FRenderGraphTimingViewSession::Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession)
{
	AnalysisSession = &InAnalysisSession;

	const FRenderGraphProvider* RenderGraphProvider = InAnalysisSession.ReadProvider<FRenderGraphProvider>(FRenderGraphProvider::ProviderName);

	if (RenderGraphProvider)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(GetAnalysisSession());

		const FRenderGraphProvider::TGraphTimeline& GraphTimeline = RenderGraphProvider->GetGraphTimeline();

		if (GraphTimeline.GetEventCount() && !Track.IsValid())
		{
			Track = MakeShared<FRenderGraphTrack>(*this);
			Track->SetVisibilityFlag(bTrackVisible);
			Track->SetOrder(MAX_int32);

			InTimingViewSession.AddScrollableTrack(Track);
			InTimingViewSession.InvalidateScrollableTracksOrder();
		}
	}
}

void FRenderGraphTimingViewSession::ExtendFilterMenu(FMenuBuilder& InMenuBuilder)
{
	InMenuBuilder.BeginSection("RenderGraphTracks", LOCTEXT("RenderGraphHeader", "RDG"));
	{
		InMenuBuilder.AddMenuEntry(
			LOCTEXT("RenderGraphTimingTracks", "RDG Tracks"),
			LOCTEXT("RenderGraphTimingTracks_Tooltip", "Show/hide the RDG track"),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateRaw(this, &FRenderGraphTimingViewSession::ToggleRenderGraphTrack),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return bTrackVisible; })
				),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	InMenuBuilder.EndSection();
}

void FRenderGraphTimingViewSession::ToggleRenderGraphTrack()
{
	bTrackVisible = !bTrackVisible;

	if (Track)
	{
		Track->SetVisibilityFlag(bTrackVisible);
	}
}

} //namespace RenderGraphInsights
} //namespace UE

#undef LOCTEXT_NAMESPACE
