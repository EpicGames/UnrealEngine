// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace TraceServices { class IAnalysisSession; }
namespace UE::Insights::Timing { class ITimingViewSession; }
namespace UE::Insights::Timing { enum class ETimeChangedFlags : int32; }
class FMenuBuilder;
class SDockTab;

namespace UE
{
namespace RenderGraphInsights
{
class FRenderGraphTrack;

class FRenderGraphTimingViewSession
{
public:
	FRenderGraphTimingViewSession() = default;

	void OnBeginSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void OnEndSession(UE::Insights::Timing::ITimingViewSession& InTimingViewSession);
	void Tick(UE::Insights::Timing::ITimingViewSession& InTimingViewSession, const TraceServices::IAnalysisSession& InAnalysisSession);
	void ExtendFilterMenu(FMenuBuilder& InMenuBuilder);

	/** Get the last cached analysis session */
	const TraceServices::IAnalysisSession& GetAnalysisSession() const { return *AnalysisSession; }

	/** Check whether the analysis session is valid */
	bool IsAnalysisSessionValid() const { return AnalysisSession != nullptr; }

	/** Show/Hide the RenderGraph track */
	void ToggleRenderGraphTrack();

	UE::Insights::Timing::ITimingViewSession* GetTimingViewSession() const { return TimingViewSession; }

private:
	const TraceServices::IAnalysisSession* AnalysisSession = nullptr;
	UE::Insights::Timing::ITimingViewSession* TimingViewSession;
	TSharedPtr<FRenderGraphTrack> Track;
	bool bTrackVisible = true;
};

} //namespace RenderGraphInsights
} //namespace UE
