// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphTraceModule.h"
#include "RenderGraphProvider.h"
#include "RenderGraphAnalyzer.h"

namespace UE
{
namespace RenderGraphInsights
{

FName FRenderGraphTraceModule::ModuleName("TraceModule_RenderGraph");

void FRenderGraphTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("RDG");
}

void FRenderGraphTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	TSharedPtr<FRenderGraphProvider> RenderGraphProvider = MakeShared<FRenderGraphProvider>(InSession);
	InSession.AddProvider(FRenderGraphProvider::ProviderName, RenderGraphProvider);

	InSession.AddAnalyzer(new FRenderGraphAnalyzer(InSession, *RenderGraphProvider));
}

void FRenderGraphTraceModule::GetLoggers(TArray<const TCHAR *>& OutLoggers)
{
	OutLoggers.Add(TEXT("RDG"));
}

void FRenderGraphTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{

}

} //namespace RenderGraphInsights
} //namespace UE
