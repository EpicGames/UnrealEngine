// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphTrack.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "PixelFormat.h"
#include "RenderGraphTimingViewSession.h"
#include "RenderGraphTrackDrawHelper.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SSpinBox.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"

// TraceInsights
#include "Insights/ITimingViewSession.h"
#include "Insights/ViewModels/TimingEvent.h"

#define LOCTEXT_NAMESPACE "RenderGraphTrack"

namespace UE { namespace RenderGraphInsights {

constexpr uint32 kBuilderColor = 0xffa0a0a0;
constexpr uint32 kRasterPassColor = 0xff7F2D2D;
constexpr uint32 kComputePassColor = 0xff2D9F9F;
constexpr uint32 kNoParameterPassColor = 0xff4D4D4D;
constexpr uint32 kAsyncComputePassColor = 0xff2D7f2d;
constexpr uint32 kTextureColor = 0xff89cff0;
constexpr uint32 kBufferColor = 0xff66D066;
constexpr uint32 kUntrackedColor = 0x2f2f2f2f;

static uint32 GetPassColor(const FPassPacket& Packet)
{
	const ERDGPassFlags Flags = Packet.Flags;

	uint32 Color = 0;

	const bool bNoParameterPass = (!Packet.Buffers.Num() && !Packet.Textures.Num());

	if (bNoParameterPass && !Packet.bCulled)
	{
		Color = kNoParameterPassColor;
	}
	else if (EnumHasAnyFlags(Flags, ERDGPassFlags::AsyncCompute))
	{
		Color = kAsyncComputePassColor;
	}
	else if (EnumHasAnyFlags(Flags, ERDGPassFlags::Raster))
	{
		Color = kRasterPassColor;
	}
	else
	{
		Color = kComputePassColor;
	}

	if (Packet.bCulled)
	{
		Color &= 0x00FFFFFF;
		Color |= 0x40000000;
	}

	return Color;
}

inline uint32 GetResourceColorBySize(uint64 Size, uint64 MaxSize)
{
	const FLinearColor Low(0.01f, 0.01f, 0.01f, 0.25f);
	const FLinearColor High(1.0, 0.1f, 0.1f, 1.0f);
	const float Percentage = FMath::Sqrt(float(Size) / float(MaxSize));
	return FLinearColor::LerpUsingHSV(Low, High, Percentage).ToFColor(false).ToPackedARGB();
}

inline uint32 GetResourceColorByTransientCache(bool bHit)
{
	static const uint32 HitColor = FLinearColor(0.01f, 0.01f, 0.01f, 0.25f).ToFColor(false).ToPackedARGB();
	static const uint32 MissColor = FLinearColor(1.0f, 0.1f, 0.1f, 1.0f).ToFColor(false).ToPackedARGB();
	return bHit ? HitColor : MissColor;
}

inline uint32 GetResourceColorByName(const TCHAR* Str)
{
	uint32 Color = 0;
	if (Str != nullptr)
	{
		for (const TCHAR* c = Str; *c; ++c)
		{
			Color = (Color + *c) * 0x2c2c57ed;
		}
	}
	return Color | 0xFF000000;
}

///////////////////////////////////////////////////////////////////////////////

template <typename VisibleItemType>
static VisibleItemType& AddVisibleItem(TArray<VisibleItemType>& VisibleItems, const VisibleItemType& InVisibleItem)
{
	const uint32 VisibleIndex = VisibleItems.Num();

	VisibleItemType& VisibleItem = VisibleItems.Emplace_GetRef(InVisibleItem);
	check(VisibleItem.GetPacket().VisibleIndex == kInvalidVisibleIndex);
	VisibleItem.GetPacket().VisibleIndex = VisibleIndex;
	VisibleItem.Index = VisibleIndex;
	return VisibleItem;
}

template <typename VisibleItemType>
static void ResetVisibleItemArray(TArray<VisibleItemType>& VisibleItems)
{
	for (VisibleItemType& VisibleItem : VisibleItems)
	{
		check(VisibleItem.GetPacket().VisibleIndex == VisibleItem.Index);
		VisibleItem.GetPacket().VisibleIndex = kInvalidVisibleIndex;
	}
	VisibleItems.Empty();
}

template <typename VisibleResourceType>
static VisibleResourceType& AddVisibleResource(TArray<VisibleResourceType>& VisibleResources, const VisibleResourceType& InVisibleResource)
{
	const uint32 VisibleIndex = VisibleResources.Num();

	VisibleResourceType& VisibleResource = VisibleResources.Emplace_GetRef(InVisibleResource);
	check(!VisibleResource.GetPacket().VisibleItems.Contains(VisibleIndex));
	VisibleResource.GetPacket().VisibleItems.Emplace(VisibleIndex);
	VisibleResource.Index = VisibleIndex;
	return VisibleResource;
}

template <typename VisibleResourceType>
static void ResetVisibleResourceArray(TArray<VisibleResourceType>& VisibleResources)
{
	for (VisibleResourceType& VisibleResource : VisibleResources)
	{
		VisibleResource.GetPacket().VisibleItems.Reset();
	}
	VisibleResources.Empty();
}

///////////////////////////////////////////////////////////////////////////////

class FRenderGraphTimingEvent : public FTimingEvent
{
	INSIGHTS_DECLARE_RTTI(FRenderGraphTimingEvent, FTimingEvent)
public:
	FRenderGraphTimingEvent(const TSharedRef<const FBaseTimingTrack> InTrack, const FVisibleItem& InItem)
		: FTimingEvent(InTrack, InItem.Packet->StartTime, InItem.Packet->EndTime, (int32)InItem.DepthH)
	{}

	virtual bool Equals(const ITimingEvent& Other) const override
	{
		if (Other.Is<FRenderGraphTimingEvent>())
		{
			return &GetPacket() == &Other.As<FRenderGraphTimingEvent>().GetPacket();
		}
		return false;
	}

	virtual const FVisibleItem& GetItem() const = 0;
	virtual const FPacket& GetPacket() const = 0;
	virtual uint32 GetAllocationIndex() const { return 0; };
};

template <typename ItemType>
class TRenderGraphTimingEventHelper : public FRenderGraphTimingEvent
{
public:
	TRenderGraphTimingEventHelper(const TSharedRef<const FBaseTimingTrack>& InTrack, const ItemType& InItem)
		: FRenderGraphTimingEvent(InTrack, InItem)
		, Item(InItem)
	{
		Item.Index = kInvalidVisibleIndex;
		Item.StartTime = GetPacket().StartTime;
		Item.EndTime = GetPacket().EndTime;
	}

	const ItemType& GetItem() const override
	{
		return Item;
	}

	const typename ItemType::PacketType& GetPacket() const final
	{
		return Item.GetPacket();
	}

private:
	ItemType Item;
};

class FVisibleScopeEvent final : public TRenderGraphTimingEventHelper<FVisibleScope>
{
	INSIGHTS_DECLARE_RTTI(FVisibleScopeEvent, FRenderGraphTimingEvent)
public:
	FVisibleScopeEvent(const TSharedRef<const FBaseTimingTrack>& InTrack, const FVisibleScope& InItem)
		: TRenderGraphTimingEventHelper<FVisibleScope>(InTrack, InItem)
	{}
};

class FVisiblePassEvent final : public TRenderGraphTimingEventHelper<FVisiblePass>
{
	INSIGHTS_DECLARE_RTTI(FVisiblePassEvent, FRenderGraphTimingEvent)
public:
	FVisiblePassEvent(const TSharedRef<const FBaseTimingTrack>& InTrack, const FVisiblePass& InItem)
		: TRenderGraphTimingEventHelper<FVisiblePass>(InTrack, InItem)
	{}
};

class FVisibleTextureEvent final : public TRenderGraphTimingEventHelper<FVisibleTexture>
{
	INSIGHTS_DECLARE_RTTI(FVisibleTextureEvent, FRenderGraphTimingEvent)
public:
	FVisibleTextureEvent(const TSharedRef<const FBaseTimingTrack>& InTrack, const FVisibleTexture& InItem)
		: TRenderGraphTimingEventHelper<FVisibleTexture>(InTrack, InItem)
	{}

	uint32 GetAllocationIndex() const override { return GetItem().AllocationIndex; };
};

class FVisibleBufferEvent final : public TRenderGraphTimingEventHelper<FVisibleBuffer>
{
	INSIGHTS_DECLARE_RTTI(FVisibleBufferEvent, FRenderGraphTimingEvent)
public:
	FVisibleBufferEvent(const TSharedRef<const FBaseTimingTrack>& InTrack, const FVisibleBuffer& InItem)
		: TRenderGraphTimingEventHelper<FVisibleBuffer>(InTrack, InItem)
	{}

	uint32 GetAllocationIndex() const override { return GetItem().AllocationIndex; };
};

inline const FPassPacket* TryGetPassPacket(const TSharedPtr<const ITimingEvent>& TimingEventBase)
{
	if (TimingEventBase)
	{
		if (TimingEventBase->Is<FVisiblePassEvent>())
		{
			return &TimingEventBase->As<FVisiblePassEvent>().GetPacket();
		}
	}
	return nullptr;
}

///////////////////////////////////////////////////////////////////////////////
// Packet Filter
///////////////////////////////////////////////////////////////////////////////

class FPacketFilter final : public ITimingEventFilter
{
	INSIGHTS_DECLARE_RTTI(FPacketFilter, ITimingEventFilter)
public:
	FPacketFilter(const TSharedPtr<const FRenderGraphTimingEvent>& InEvent)
		: Event(InEvent)
		, Packet(&InEvent->GetPacket())
		, Graph(Packet->Graph)
		, AllocationIndex(InEvent->GetAllocationIndex())
	{}
	~FPacketFilter() {}

	bool FilterPacket(const FPacket& PacketToFilter) const
	{
		if (FilterPacketExact(PacketToFilter))
		{
			return true;
		}

		if (PacketToFilter.Graph != Graph)
		{
			return false;
		}

		if (Packet->Is<FScopePacket>())
		{
			const FScopePacket& Scope = Packet->As<FScopePacket>();

			if (PacketToFilter.Is<FPassIntervalPacket>())
			{
				const FPassIntervalPacket& PassIntervalToFilter = PacketToFilter.As<FPassIntervalPacket>();
				return Intersects(Scope, PassIntervalToFilter);
			}
			else if (PacketToFilter.Is<FPassPacket>())
			{
				const FPassPacket& PassToFilter = PacketToFilter.As<FPassPacket>();
				return Intersects(Scope, PassToFilter);
			}
		}

		return false;
	}

	bool FilterPacketExact(const FPacket& PacketToFilter) const
	{
		return Packet == &PacketToFilter;
	}

	const FPacket& GetPacket() const
	{
		return *Packet;
	}

	const FGraphPacket& GetGraph() const
	{
		return *Graph;
	}

	uint32 GetAllocationIndex() const
	{
		return AllocationIndex;
	}

private:
	//! ITimingEventFilter
	bool FilterTrack(const FBaseTimingTrack& InTrack) const override { return true; }
	bool FilterEvent(const ITimingEvent& InEvent) const override { return true; }
	bool FilterEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0) const override { return true; }
	uint32 GetChangeNumber() const override { return 0; }
	//! ITimingEventFilter

	TSharedPtr<const FRenderGraphTimingEvent> Event;
	const FPacket* Packet{};
	const FGraphPacket* Graph{};
	uint32 AllocationIndex = 0;
};

inline bool TryFilterPacket(const FPacketFilter* Filter, const FPacket& Packet)
{
	return Filter ? Filter->FilterPacket(Packet) : false;
}

inline bool TryFilterPacketExact(const FPacketFilter* Filter, const FPacket& Packet)
{
	return Filter ? Filter->FilterPacketExact(Packet) : false;
}

inline const FPacketFilter* GetPacketFilter(const TSharedPtr<ITimingEventFilter>& EventFilter)
{
	if (EventFilter && EventFilter->Is<FPacketFilter>())
	{
		return &EventFilter->As<FPacketFilter>();
	}
	return nullptr;
}

///////////////////////////////////////////////////////////////////////////////

FBoundingBox FVisibleItem::GetBoundingBox(const FTimingViewLayout& Layout) const
{
	return FBoundingBox(FVector2D(SlateX, DepthY), FVector2D(SlateX + SlateW, DepthY + DepthH));
}

void FVisibleItem::Draw(FRenderGraphTrackDrawStateBuilder& Builder) const
{
	if (DepthH != 1.0f || FMath::Fractional(DepthY) != 0.0f)
	{
		Builder.AddEvent(StartTime, EndTime, DepthY, DepthH, Color, [this](float) { return Name; });
	}
	else
	{
		Builder.AddEvent(StartTime, EndTime, DepthY, Color, [this](float) { return Name; });
	}
}

///////////////////////////////////////////////////////////////////////////////

void FVisibleGraph::Reset()
{
	ResetVisibleItemArray(Scopes);
	ResetVisibleItemArray(Passes);
	ResetVisibleResourceArray(Textures);
	ResetVisibleResourceArray(Buffers);
}

void FVisibleGraph::AddScope(const FVisibleScope& VisibleScope)
{
	AddVisibleItem(Scopes, VisibleScope);
}

void FVisibleGraph::AddPass(const FVisiblePass& InVisiblePass)
{
	const FVisiblePass& VisiblePass = AddVisibleItem(Passes, InVisiblePass);

	if (EnumHasAnyFlags(VisiblePass.GetPacket().Flags, ERDGPassFlags::AsyncCompute))
	{
		AsyncComputePasses.Add(VisiblePass.Index);
	}
}

void FVisibleGraph::AddTexture(const FVisibleTexture& VisibleTexture)
{
	AddVisibleResource(Textures, VisibleTexture);

	DepthH = FMath::Max(DepthH, VisibleTexture.DepthY + VisibleTexture.DepthH);
}

void FVisibleGraph::AddBuffer(const FVisibleBuffer& VisibleBuffer)
{
	AddVisibleResource(Buffers, VisibleBuffer);

	DepthH = FMath::Max(DepthH, VisibleBuffer.DepthY + VisibleBuffer.DepthH);
}

const FVisibleItem* FVisibleGraph::FindItem(float InSlateX, float InDepthY) const
{
	for (const FVisibleTexture& Texture : Textures)
	{
		if (Texture.Intersects(InSlateX, InDepthY))
		{
			return &Texture;
		}
	}

	for (const FVisibleBuffer& Buffer : Buffers)
	{
		if (Buffer.Intersects(InSlateX, InDepthY))
		{
			return &Buffer;
		}
	}

	for (const FVisibleScope& Scope : Scopes)
	{
		if (Scope.Intersects(InSlateX, InDepthY))
		{
			return &Scope;
		}
	}

	for (const FVisiblePass& Pass : Passes)
	{
		if (Pass.Intersects(InSlateX, InDepthY))
		{
			return &Pass;
		}
	}

	return nullptr;
}

void FVisibleGraph::Draw(FRenderGraphTrackDrawStateBuilder& Builder) const
{
	Builder.AddEvent(StartTime, EndTime, 0, Name, 0, kBuilderColor);
}

///////////////////////////////////////////////////////////////////////////////

INSIGHTS_IMPLEMENT_RTTI(FRenderGraphTimingEvent)
INSIGHTS_IMPLEMENT_RTTI(FPacketFilter)
INSIGHTS_IMPLEMENT_RTTI(FVisibleScopeEvent)
INSIGHTS_IMPLEMENT_RTTI(FVisiblePassEvent)
INSIGHTS_IMPLEMENT_RTTI(FVisibleTextureEvent)
INSIGHTS_IMPLEMENT_RTTI(FVisibleBufferEvent)
INSIGHTS_IMPLEMENT_RTTI(FRenderGraphTrack)

///////////////////////////////////////////////////////////////////////////////

FRenderGraphTrack::FRenderGraphTrack(const FRenderGraphTimingViewSession& InSharedData)
	: Super(LOCTEXT("TrackNameFormat", "RDG").ToString())
	, SharedData(InSharedData)
	, NumLanes(0)
	, DrawState(MakeShared<FRenderGraphTrackDrawState>())
	, FilteredDrawState(MakeShared<FRenderGraphTrackDrawState>())
{
	SetValidLocations(ETimingTrackLocation::Scrollable | ETimingTrackLocation::TopDocked | ETimingTrackLocation::BottomDocked);
}

uint32 FRenderGraphTrack::GetTextureColor(const FTexturePacket& Texture, uint64 MaxSizeInBytes) const
{
	if (Texture.bTransientUntracked)
	{
		return kUntrackedColor;
	}
	if (ResourceColor == EResourceColor::Name)
	{
		return GetResourceColorByName(*Texture.Name);
	}
	if (ResourceColor == EResourceColor::Type)
	{
		return kTextureColor;
	}
	if (ResourceColor == EResourceColor::TransientCache)
	{
		return GetResourceColorByTransientCache(!Texture.bTransient || Texture.bTransientCacheHit);
	}
	return GetResourceColorBySize(Texture.SizeInBytes, MaxSizeInBytes);
}

uint32 FRenderGraphTrack::GetBufferColor(const FBufferPacket& Buffer, uint64 MaxSizeInBytes) const
{
	if (Buffer.bTransientUntracked)
	{
		return kUntrackedColor;
	}
	if (ResourceColor == EResourceColor::Name)
	{
		return GetResourceColorByName(*Buffer.Name);
	}
	if (ResourceColor == EResourceColor::Type)
	{
		return kBufferColor;
	}
	if (ResourceColor == EResourceColor::TransientCache)
	{
		return GetResourceColorByTransientCache(!Buffer.bTransient || Buffer.bTransientCacheHit);
	}
	return GetResourceColorBySize(Buffer.SizeInBytes, MaxSizeInBytes);
}

float FRenderGraphTrack::TransientByteOffsetToDepth(uint64 MemoryOffset) const
{
	if (MemoryOffset == 0)
	{
		return 0;
	}
	else
	{
		return FMath::Max((float)MemoryOffset / TransientBytesPerDepthSlot, 1.0f / 16.0f);
	}
}

float FRenderGraphTrack::TransientByteOffsetToSlateY(const FTimingViewLayout& Layout, uint64 MemoryOffset) const
{
	return (Layout.EventH + Layout.EventDY) * TransientByteOffsetToDepth(MemoryOffset);
}

void FRenderGraphTrack::Reset()
{
	FBaseTimingTrack::Reset();

	NumLanes = 0;
	DrawState->Reset();
	FilteredDrawState->Reset();
}

void FRenderGraphTrack::PreUpdate(const ITimingTrackUpdateContext& Context)
{
	const FTimingTrackViewport& Viewport = Context.GetViewport();
	const FTimingViewLayout& Layout = Viewport.GetLayout();

	if (IsDirty() || Viewport.IsHorizontalViewportDirty())
	{
		ClearDirtyFlag();

		for (FVisibleGraph& VisibleGraph : VisibleGraphs)
		{
			VisibleGraph.Reset();
		}
		ResetVisibleItemArray(VisibleGraphs);

		int32 MaxDepth = -1;

		{
			FRenderGraphTrackDrawStateBuilder Builder(*DrawState, Viewport);

			BuildDrawState(Builder, Context);

			Builder.Flush();

			if (Builder.GetMaxDepth() > MaxDepth)
			{
				MaxDepth = Builder.GetMaxDepth();
			}
		}

		const TSharedPtr<ITimingEventFilter> EventFilter = Context.GetEventFilter();
		if ((EventFilter.IsValid() && EventFilter->FilterTrack(*this)))
		{
			const bool bFastLastBuild = FilteredDrawStateInfo.LastBuildDuration < 0.005; // LastBuildDuration < 5ms
			const bool bFilterPointerChanged = !FilteredDrawStateInfo.LastEventFilter.HasSameObject(EventFilter.Get());

			if (bFastLastBuild || bFilterPointerChanged)
			{
				FilteredDrawStateInfo.LastEventFilter = EventFilter;
				FilteredDrawStateInfo.ViewportStartTime = Viewport.GetStartTime();
				FilteredDrawStateInfo.ViewportScaleX = Viewport.GetScaleX();
				FilteredDrawStateInfo.Counter = 0;
			}
			else
			{
				if (FilteredDrawStateInfo.ViewportStartTime == Viewport.GetStartTime() &&
					FilteredDrawStateInfo.ViewportScaleX == Viewport.GetScaleX())
				{
					if (FilteredDrawStateInfo.Counter > 0)
					{
						FilteredDrawStateInfo.Counter--;
					}
				}
				else
				{
					FilteredDrawStateInfo.ViewportStartTime = Viewport.GetStartTime();
					FilteredDrawStateInfo.ViewportScaleX = Viewport.GetScaleX();
					FilteredDrawStateInfo.Counter = 1; // wait
				}
			}

			if (FilteredDrawStateInfo.Counter == 0)
			{
				const uint64 StartTime = FPlatformTime::Cycles64();
				{
					FRenderGraphTrackDrawStateBuilder Builder(*FilteredDrawState, Viewport);
					BuildFilteredDrawState(Builder, Context);
					Builder.Flush();
				}
				const uint64 EndTime = FPlatformTime::Cycles64();
				FilteredDrawStateInfo.LastBuildDuration = static_cast<double>(EndTime - StartTime) * FPlatformTime::GetSecondsPerCycle64();
			}
			else
			{
				FilteredDrawState->Reset();
				FilteredDrawStateInfo.Opacity = 0.0f;
				SetDirtyFlag();
			}
		}
		else
		{
			FilteredDrawStateInfo.LastBuildDuration = 0.0;

			if (FilteredDrawStateInfo.LastEventFilter.IsValid())
			{
				FilteredDrawStateInfo.LastEventFilter.Reset();
				FilteredDrawStateInfo.Counter = 0;
				FilteredDrawState->Reset();
			}
		}

		NumLanes = MaxDepth + 1;
	}

	const float CurrentTrackHeight = GetHeight();
	const float DesiredTrackHeight = Layout.ComputeTrackHeight(NumLanes);

	if (CurrentTrackHeight < DesiredTrackHeight)
	{
		float NewTrackHeight;
		if (Viewport.IsDirty(ETimingTrackViewportDirtyFlags::VLayoutChanged))
		{
			NewTrackHeight = DesiredTrackHeight;
		}
		else
		{
			NewTrackHeight = FMath::CeilToFloat(CurrentTrackHeight * 0.9f + DesiredTrackHeight * 0.1f);
		}
		SetHeight(NewTrackHeight);
	}
	else if (CurrentTrackHeight > DesiredTrackHeight)
	{
		float NewTrackHeight;
		if (Viewport.IsDirty(ETimingTrackViewportDirtyFlags::VLayoutChanged))
		{
			NewTrackHeight = DesiredTrackHeight;
		}
		else
		{
			NewTrackHeight = FMath::FloorToFloat(CurrentTrackHeight * 0.9f + DesiredTrackHeight * 0.1f);
		}
		SetHeight(NewTrackHeight);
	}
}

void FRenderGraphTrack::Update(const ITimingTrackUpdateContext& Context)
{
	const FTimingTrackViewport& Viewport = Context.GetViewport();
	const FTimingViewLayout& Layout = Viewport.GetLayout();

	const FVector2D MousePosition = Context.GetMousePosition();
	MouseSlateX = MousePosition.X;
	MouseSlateY = MousePosition.Y - GetPosY();
	MouseDepthY = GetDepthFromLaneY(Layout, MouseSlateY);

	const TSharedPtr<const ITimingEvent> SelectedEventPtr = Context.GetSelectedEvent();
	if (SelectedEventPtr.IsValid() && SelectedEventPtr->Is<FRenderGraphTimingEvent>())
	{
		const FRenderGraphTimingEvent& SelectedEvent = SelectedEventPtr->As<FRenderGraphTimingEvent>();

		const FVector2D ViewportPos(Viewport.GetWidth(), Viewport.GetHeight());

		InitTooltip(SelectedTooltipState, SelectedEvent);
		SelectedTooltipState.SetDesiredOpacity(0.75f);
		SelectedTooltipState.SetPosition(ViewportPos, 0.0f, Viewport.GetWidth(), 0.0f, Viewport.GetHeight());
		SelectedTooltipState.Update();
	}
	else
	{
		SelectedTooltipState.Reset();
	}
}

void FRenderGraphTrack::PostUpdate(const ITimingTrackUpdateContext& Context)
{
	constexpr float HeaderWidth = 100.0f;
	constexpr float HeaderHeight = 14.0f;

	const float MouseY = Context.GetMousePosition().Y;
	if (MouseY >= GetPosY() && MouseY < GetPosY() + GetHeight())
	{
		const float MouseX = Context.GetMousePosition().X;
		SetHeaderHoveredState(MouseX < HeaderWidth&& MouseY < GetPosY() + HeaderHeight);
		SetHoveredState(true);
	}
	else
	{
		SetHoveredState(false);
	}
}

void FRenderGraphTrack::Draw(const ITimingTrackDrawContext& Context) const
{
	FRenderGraphTrackDrawHelper DrawHelper(Context);

	const bool bFilterActive = Context.GetEventFilter().IsValid() && Context.GetEventFilter()->FilterTrack(*this);

	if (bFilterActive)
	{
		DrawHelper.DrawFadedEvents(*DrawState, *this, 0.1f);

		if (FilteredDrawStateInfo.Opacity == 1.0f)
		{
			DrawHelper.DrawEvents(*FilteredDrawState, *this);
		}
		else
		{
			FilteredDrawStateInfo.Opacity = FMath::Min(1.0f, FilteredDrawStateInfo.Opacity + 0.05f);

			DrawHelper.DrawFadedEvents(*FilteredDrawState, *this, FilteredDrawStateInfo.Opacity);
		}
	}
	else
	{
		DrawHelper.DrawEvents(*DrawState, *this);
	}

	DrawHelper.DrawTrackHeader(*this);

	const FTimingTrackViewport& Viewport = Context.GetViewport();
	const FTimingViewLayout& Layout = Viewport.GetLayout();
	const FDrawContext& DrawContext = Context.GetDrawContext();

	const FLinearColor EdgeColor = DrawHelper.GetEdgeColor();
	const float ViewportWidth = Viewport.GetWidth();

	const FPassPacket* SelectedPass = TryGetPassPacket(Context.GetSelectedEvent());
	const FPassPacket* HoveredPass = TryGetPassPacket(Context.GetHoveredEvent());
	const FPacketFilter* PacketFilter = GetPacketFilter(Context.GetEventFilter());

	for (const FVisibleGraph& VisibleGraph : VisibleGraphs)
	{
		if (VisibleGraph.IsCulled())
		{
			continue;
		}

		const FGraphPacket& Graph = VisibleGraph.GetPacket();
		const float GraphWidth = VisibleGraph.SlateW;

		////////////////////////////////////////////////////////////////////////////////////
		// Draw pass columns and markers.
		////////////////////////////////////////////////////////////////////////////////////

		const float PassLineStrideMin = 5.0f;
		const float PassLineStride = GraphWidth / float(Graph.PassCount);

		if (GraphWidth >= PassLineStrideMin)
		{
			DrawHelper.DrawBox(*this, VisibleGraph.SlateX, 0.0f, 1.0f, VisibleGraph.DepthH, EdgeColor, EDrawLayer::Background);
			DrawHelper.DrawBox(*this, VisibleGraph.SlateX + VisibleGraph.SlateW, 0.0f, 1.0f, VisibleGraph.DepthH, EdgeColor, EDrawLayer::Background);
		}

		float RenderPassMergeMinX = 0.0f;
		float ParallelExecuteMinX = 0.0f;

		constexpr uint32 BarCount = 2; // Render Pass Merge + Parallel Execute
		constexpr float BarDepth  = 0.3f;
		constexpr float BarMargin = (1.0f / float(BarCount + 1)) - BarDepth;

		for (uint32 PassIndex = 0; PassIndex < Graph.PassCount; ++PassIndex)
		{
			const FPassPacket& Pass = Graph.Passes[PassIndex];
			const FVisiblePass& VisiblePass = VisibleGraph.GetVisiblePass(Pass);
			const FBoundingBox VisiblePassBox = VisiblePass.GetBoundingBox(Layout);

			if (Pass.bParallelExecuteBegin)
			{
				ParallelExecuteMinX = VisiblePassBox.Min.X;
			}

			if (Pass.bParallelExecuteEnd)
			{
				const float W = VisiblePassBox.Max.X - ParallelExecuteMinX;
				const float H = BarDepth;
				const float X = ParallelExecuteMinX;
				const float Y = VisibleGraph.HeaderPassBarDepth + BarMargin;

				DrawHelper.DrawBox(*this, X, Y, W, H, FLinearColor(1.0f, 1.0f, 1.0f, 0.75f), EDrawLayer::Background);
			}

			if (!Pass.bSkipRenderPassBegin && Pass.bSkipRenderPassEnd)
			{
				RenderPassMergeMinX = VisiblePassBox.Min.X;
			}

			if (Pass.bSkipRenderPassBegin && !Pass.bSkipRenderPassEnd)
			{
				const float W = VisiblePassBox.Max.X - RenderPassMergeMinX;
				const float H = BarDepth;
				const float X = RenderPassMergeMinX;
				const float Y = VisibleGraph.HeaderPassBarDepth + BarMargin * 2.0f + BarDepth;

				DrawHelper.DrawBox(*this, X, Y, W, H, FLinearColor(0.8f, 0.2f, 0.2f, 0.75f), EDrawLayer::Background);
			}

			if (Pass.bParallelExecuteAsyncAllowed)
			{
				const float W = VisiblePassBox.Max.X - VisiblePassBox.Min.X;
				const float H = BarDepth;
				const float X = VisiblePassBox.Min.X;
				const float Y = VisibleGraph.HeaderPassBarDepth + BarMargin * 3.0f + BarDepth;

				DrawHelper.DrawBox(*this, X, Y, W, H, FLinearColor(0.2f, 0.2f, 0.8f, 0.75f), EDrawLayer::Background);
			}

			const float X = Viewport.TimeToSlateUnitsRounded(Pass.StartTime);
			const float Y = VisiblePassBox.Max.Y;
			const float H = VisibleGraph.DepthH - VisiblePassBox.Max.Y;

			const bool bHoveredPass = HoveredPass == &Pass;
			const bool bSelectedPass = SelectedPass == &Pass;
			const bool bFilteredPass = TryFilterPacket(PacketFilter, Pass);

			if (bHoveredPass || bFilteredPass || bSelectedPass)
			{
				const float W = PassLineStride + 1;
				const FLinearColor EdgeColorTranslucent = FLinearColor(EdgeColor.R, EdgeColor.G, EdgeColor.B, bFilteredPass ? 1.0f : 0.5f);

				DrawHelper.DrawBox(*this, X, Y, W, H, EdgeColorTranslucent, EDrawLayer::Background);
			}
			else if (PassLineStride >= PassLineStrideMin && PassIndex != 0)
			{
				const float W = 1.0f;

				DrawHelper.DrawBox(*this, X, Y, W, H, EdgeColor, EDrawLayer::Background);
			}
		}

		////////////////////////////////////////////////////////////////////////////////////
		// Draw async compute fork / join fences
		////////////////////////////////////////////////////////////////////////////////////

		for (uint32 VisibleIndex : VisibleGraph.AsyncComputePasses)
		{
			const FVisiblePass& AsyncComputeVisiblePass = VisibleGraph.Passes[VisibleIndex];
			const FPassPacket& AsyncComputePass = AsyncComputeVisiblePass.GetPacket();

			float TintAlpha = 0.25f;

			if (!bFilterActive || TryFilterPacket(PacketFilter, AsyncComputePass))
			{
				TintAlpha = 0.75f;
			}

			const float StartT = 0.2f;
			const float EndT = 1.0f - StartT;
			const float SplineDir = 20.0f;

			const FRDGPassHandle GraphicsForkHandle = AsyncComputePass.GraphicsForkPass;

			if (GraphicsForkHandle.IsValid() && AsyncComputePass.bAsyncComputeBegin)
			{
				const FPassPacket& GraphicsForkPass = *Graph.GetPass(AsyncComputePass.GraphicsForkPass);
				const FVisiblePass& GraphicsForkVisiblePass = VisibleGraph.GetVisiblePass(GraphicsForkPass);

				const float X = GraphicsForkVisiblePass.SlateX + GraphicsForkVisiblePass.SlateW * EndT;
				const float Y = GraphicsForkVisiblePass.DepthY + GraphicsForkVisiblePass.DepthH;
				const float LocalEndX = AsyncComputeVisiblePass.SlateX + (AsyncComputeVisiblePass.SlateW * StartT) - X;
				const float LocalEndY = AsyncComputeVisiblePass.DepthY - Y;

				FSplinePrimitive Spline;
				Spline.Start.X = X;
				Spline.Start.Y = Y;
				Spline.StartDir = FVector2D(0, SplineDir);
				Spline.End = FVector2D(LocalEndX, LocalEndY);
				Spline.EndDir = FVector2D(0, SplineDir);
				Spline.Thickness = 2.0f;
				Spline.Tint = FLinearColor(0.4f, 1.0f, 0.4f, TintAlpha);
				DrawHelper.DrawSpline(*this, Spline, EDrawLayer::EventFill);
			}

			const FRDGPassHandle GraphicsJoinHandle = AsyncComputePass.GraphicsJoinPass;

			if (GraphicsJoinHandle.IsValid() && AsyncComputePass.bAsyncComputeEnd)
			{
				const FPassPacket& GraphicsJoinPass = *Graph.GetPass(AsyncComputePass.GraphicsJoinPass);
				const FVisiblePass& GraphicsJoinVisiblePass = VisibleGraph.GetVisiblePass(GraphicsJoinPass);

				const float X = AsyncComputeVisiblePass.SlateX + AsyncComputeVisiblePass.SlateW * EndT;
				const float Y = AsyncComputeVisiblePass.DepthY;
				const float LocalEndX = GraphicsJoinVisiblePass.SlateX + (GraphicsJoinVisiblePass.SlateW * StartT) - X;
				const float LocalEndY = GraphicsJoinVisiblePass.DepthY + GraphicsJoinVisiblePass.DepthH - Y;

				FSplinePrimitive Spline;
				Spline.Start.X = X;
				Spline.Start.Y = Y;
				Spline.StartDir = FVector2D(0, -SplineDir);
				Spline.End = FVector2D(LocalEndX, LocalEndY);
				Spline.EndDir = FVector2D(0, -SplineDir);
				Spline.Thickness = 2.0f;
				Spline.Tint = FLinearColor(1.0f, 0.4f, 0.4f, TintAlpha);
				DrawHelper.DrawSpline(*this, Spline, EDrawLayer::EventFill);
			}
		}

		////////////////////////////////////////////////////////////////////////////////////
		// Draw transient memory vertical lines and size markers
		////////////////////////////////////////////////////////////////////////////////////

		if (Visualizer == EVisualizer::TransientMemory)
		{
			const FLinearColor MBLineColor(0.8f, 0.1f, 0.1f, 0.2f);
			const FLinearColor MBTextColor(0.8f, 0.1f, 0.1f, 0.8f);

			const float StartDepth = VisibleGraph.HeaderEndDepth;

			const bool bDrawText = GraphWidth > 100.0f;

			const uint64 MiB = 1024 * 1024;

			const auto DrawMibText = [&](float Depth, uint64 Offset, uint32 MemoryRangeIndex, uint32 MemoryRangeSizeInMB, FRHITransientAllocationStats::EMemoryRangeFlags MemoryRangeFlags)
			{
				FString Text;
				FLinearColor TextColor = MBTextColor;

				if (Offset != 0)
				{
					Text = FString::Printf(TEXT("%0.2fMiB"), (float)Offset / MiB);
				}
				else
				{
					Text = FString::Printf(TEXT("Memory Range [%d] %uMiB"), MemoryRangeIndex, MemoryRangeSizeInMB);

					if (EnumHasAnyFlags(MemoryRangeFlags, FRHITransientAllocationStats::EMemoryRangeFlags::FastVRAM))
					{
						Text += TEXT("(FastVRAM)");
					}

					TextColor.A = 1.0f;
				}

				const FSlateFontInfo& Font = DrawHelper.GetEventFont();
				const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
				const FVector2D TextExtent = FontMeasureService->Measure(Text, Font);

				const float X = FMath::Max(VisibleGraph.SlateX - 2.0f - TextExtent.X, 2.0f);
				const float Y = GetPosY() + GetLaneY(Layout, Depth) - TextExtent.Y * 0.5f;

				DrawContext.DrawText(DrawHelper.GetFirstLayerId() + int32(EDrawLayer::Background), X, Y, Text, Font, TextColor);
				return X + TextExtent.X + 2.0f;
			};

			const float SinglePixelDepth = 1.0f / Layout.EventH;

			for (int32 MemoryRangeIndex = 0; MemoryRangeIndex < Graph.TransientMemoryRangeByteOffsets.Num(); ++MemoryRangeIndex)
			{
				const float MemoryRangeStartDepth = StartDepth + VisibleGraph.TransientMemoryRangeDepthOffsets[MemoryRangeIndex];

				const auto& TransientMemoryRange = Graph.TransientAllocationStats.MemoryRanges[MemoryRangeIndex];

				uint64 Offset = 0;
				uint32 OffsetIndex = 0;

				const uint32 TicksPerText = 4;

				while (Offset < TransientMemoryRange.CommitSize)
				{
					const float Depth = MemoryRangeStartDepth + TransientByteOffsetToDepth(Offset);

					FLinearColor LineColor = MBLineColor;

					float LineStartX = VisibleGraph.SlateX;
					float LineWidth  = VisibleGraph.SlateW;

					const bool bMajorTick = (OffsetIndex % TicksPerText) == 0;

					if (bMajorTick)
					{
						LineColor.A = 0.5f;
					}

					if (bDrawText && bMajorTick)
					{
						LineStartX = DrawMibText(Depth, Offset, MemoryRangeIndex, TransientMemoryRange.Capacity / MiB, TransientMemoryRange.Flags);
						LineWidth -= LineStartX - VisibleGraph.SlateX;
					}

					DrawHelper.DrawBox(*this, LineStartX, Depth, LineWidth, SinglePixelDepth, LineColor, EDrawLayer::Background);

					Offset += TransientBytesPerDepthSlot * (Layout.bIsCompactMode ? 3 : 1);
					OffsetIndex++;
				}
			}
		}
	}
}

void FRenderGraphTrack::PostDraw(const ITimingTrackDrawContext& Context) const
{
	SelectedTooltipState.Draw(Context.GetDrawContext());
}

void FRenderGraphTrack::BuildDrawState(FRenderGraphTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const float kMinGraphPixels = 5.0f;
	const float kMinPassMarginPixels = 5.0f;

	const FRenderGraphProvider* RenderGraphProvider = SharedData.GetAnalysisSession().ReadProvider<FRenderGraphProvider>(FRenderGraphProvider::ProviderName);

	if (!RenderGraphProvider)
	{
		return;
	}

	const FTimingTrackViewport& Viewport = Context.GetViewport();
	const FTimingViewLayout& Layout = Viewport.GetLayout();

	const auto DrawPasses = [&](double GraphStartTime, double GraphEndTime, FVisibleGraph& VisibleGraph, double& SinglePixelTimeMargin, uint32& DepthOffset)
	{
		SinglePixelTimeMargin = Viewport.GetDurationForViewportDX(1.0);

		const FGraphPacket& GraphPacket = VisibleGraph.GetPacket();

		if (Viewport.GetViewportDXForDuration(GraphPacket.NormalizedPassDuration) <= kMinPassMarginPixels)
		{
			SinglePixelTimeMargin = 0.0;
		}

		DepthOffset = 1;

		for (uint32 ScopeIndex = 0, ScopeCount = GraphPacket.Scopes.Num(); ScopeIndex < ScopeCount; ++ScopeIndex)
		{
			const FScopePacket& Scope = GraphPacket.Scopes[ScopeIndex];
			const double StartTime = Scope.StartTime + SinglePixelTimeMargin;
			const double EndTime = Scope.EndTime;

			const FVisibleScope VisibleScope(Viewport, Scope, FTimingEvent::ComputeEventColor(*Scope.Name), StartTime, EndTime, DepthOffset + Scope.Depth);
			VisibleScope.Draw(Builder);
			VisibleGraph.AddScope(VisibleScope);
		}

		if (GraphPacket.ScopeDepth)
		{
			DepthOffset += GraphPacket.ScopeDepth + 1;
		}

		VisibleGraph.HeaderPassBarDepth = DepthOffset++;

		bool bAnyAsyncCompute = false;

		for (uint32 PassIndex = 0, PassCount = GraphPacket.Passes.Num(); PassIndex < PassCount; ++PassIndex)
		{
			const FPassPacket& Pass = GraphPacket.Passes[PassIndex];
			const double StartTime = Pass.StartTime + SinglePixelTimeMargin;
			const double EndTime = Pass.EndTime;
			const bool bAsyncCompute = EnumHasAnyFlags(Pass.Flags, ERDGPassFlags::AsyncCompute);
			const uint32 Depth = DepthOffset + (bAsyncCompute ? 2 : 0);
			const uint32 Color = GetPassColor(Pass);

			const FVisiblePass VisiblePass(Viewport, Pass, Color, StartTime, EndTime, Depth);
			VisiblePass.Draw(Builder);
			VisibleGraph.AddPass(VisiblePass);

			bAnyAsyncCompute |= bAsyncCompute;
		}

		// Empty space between passes / resources.
		DepthOffset += bAnyAsyncCompute ? 3 : 1;

		VisibleGraph.HeaderEndDepth = DepthOffset;
		VisibleGraph.DepthH = DepthOffset;
	};

	if (Visualizer == EVisualizer::TransientMemory)
	{
		TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData.GetAnalysisSession());

		const FRenderGraphProvider::TGraphTimeline& GraphTimeline = RenderGraphProvider->GetGraphTimeline();
		GraphTimeline.EnumerateEvents(Viewport.GetStartTime(), Viewport.GetEndTime(),
			[&](double GraphStartTime, double GraphEndTime, uint32, const TSharedPtr<FGraphPacket>& Graph)
		{
			FVisibleGraph& VisibleGraph = AddVisibleGraph(Viewport, *Graph, kBuilderColor);
			VisibleGraph.Draw(Builder);

			if (Viewport.GetViewportDXForDuration(GraphEndTime - GraphStartTime) <= kMinGraphPixels)
			{
				return TraceServices::EEventEnumerate::Continue;
			}

			uint32 DepthOffset;
			double SinglePixelTimeMargin;
			DrawPasses(GraphStartTime, GraphEndTime, VisibleGraph, SinglePixelTimeMargin, DepthOffset);

			const float YOffset = Layout.GetLaneY(DepthOffset);

			const auto IsPacketCulled = [&](const FResourcePacket& Packet)
			{
				if (!Packet.bTransient)
				{
					return true;
				}

				if (!Packet.bTransientUntracked && !ShowTracked())
				{
					return true;
				}

				if (FilterSize > 0.0f && Packet.SizeInBytes < uint64(FilterSize * 1024.0f * 1024.0f))
				{
					return true;
				}

				if (!FilterText.IsEmpty() && !Packet.Name.Contains(FilterText))
				{
					return true;
				}

				return false;
			};

			uint64 MaxSizeInBytes = 0;

			for (uint32 TextureIndex = 0, TextureCount = Graph->Textures.Num(); TextureIndex < TextureCount; ++TextureIndex)
			{
				FTexturePacket& Texture = Graph->Textures[TextureIndex];

				if (!IsPacketCulled(Texture))
				{
					MaxSizeInBytes = FMath::Max(MaxSizeInBytes, Texture.SizeInBytes);
				}
			}

			for (uint32 BufferIndex = 0, BufferCount = Graph->Buffers.Num(); BufferIndex < BufferCount; ++BufferIndex)
			{
				FBufferPacket& Buffer = Graph->Buffers[BufferIndex];

				if (!IsPacketCulled(Buffer))
				{
					MaxSizeInBytes = FMath::Max(MaxSizeInBytes, Buffer.SizeInBytes);
				}
			}

			for (uint32 TextureIndex = 0, TextureCount = Graph->Textures.Num(); TextureIndex < TextureCount; ++TextureIndex)
			{
				FTexturePacket& Texture = Graph->Textures[TextureIndex];

				if (IsPacketCulled(Texture))
				{
					continue;
				}

				for (int32 AllocationIndex = 0; AllocationIndex < Texture.TransientAllocations.Num(); ++AllocationIndex)
				{
					const FRHITransientAllocationStats::FAllocation& Allocation = Texture.TransientAllocations[AllocationIndex];

					const float MemoryRangeOffsetDepthBase = VisibleGraph.TransientMemoryRangeDepthOffsets[Allocation.MemoryRangeIndex];

					const float DepthY = TransientByteOffsetToDepth(Allocation.OffsetMin) + MemoryRangeOffsetDepthBase + DepthOffset;
					const float DepthH = TransientByteOffsetToDepth(Allocation.OffsetMax - Allocation.OffsetMin);
					const double StartTime = Texture.StartTime + SinglePixelTimeMargin;
					const double EndTime = Texture.EndTime;
					const uint32 Color = GetTextureColor(Texture, MaxSizeInBytes);

					const FVisibleTexture VisibleTexture(Viewport, Texture, Color, StartTime, EndTime, DepthY, DepthH, AllocationIndex);
					VisibleTexture.Draw(Builder);
					VisibleGraph.AddTexture(VisibleTexture);
				}
			}

			for (uint32 BufferIndex = 0, BufferCount = Graph->Buffers.Num(); BufferIndex < BufferCount; ++BufferIndex)
			{
				FBufferPacket& Buffer = Graph->Buffers[BufferIndex];

				if (IsPacketCulled(Buffer))
				{
					continue;
				}

				for (int32 AllocationIndex = 0; AllocationIndex < Buffer.TransientAllocations.Num(); ++AllocationIndex)
				{
					const FRHITransientAllocationStats::FAllocation& Allocation = Buffer.TransientAllocations[AllocationIndex];

					const float MemoryRangeOffsetDepthBase = VisibleGraph.TransientMemoryRangeDepthOffsets[Allocation.MemoryRangeIndex];

					const float DepthY = TransientByteOffsetToDepth(Allocation.OffsetMin) + MemoryRangeOffsetDepthBase + DepthOffset;
					const float DepthH = TransientByteOffsetToDepth(Allocation.OffsetMax - Allocation.OffsetMin);
					const double StartTime = Buffer.StartTime + SinglePixelTimeMargin;
					const double EndTime = Buffer.EndTime;
					const uint32 Color = GetBufferColor(Buffer, MaxSizeInBytes);

					const FVisibleBuffer VisibleBuffer(Viewport, Buffer, Color, StartTime, EndTime, DepthY, DepthH, AllocationIndex);
					VisibleBuffer.Draw(Builder);
					VisibleGraph.AddBuffer(VisibleBuffer);
				}
			}

			return TraceServices::EEventEnumerate::Continue;
		});
	}
	else if (Visualizer == EVisualizer::Resources)
	{
		struct FResourceEntry
		{
			double StartTime{};
			double EndTime{};
			uint64 SizeInBytes{};
			uint32 Index{};
			uint32 Order{};
			ERDGViewableResourceType Type = ERDGViewableResourceType::Texture;
			bool bHasPreviousOwner{};
		};

		TArray<FResourceEntry> Resources;
		TArray<uint16> TextureIndexToDepth;
		TArray<uint16> BufferIndexToDepth;

		TraceServices::FAnalysisSessionReadScope SessionReadScope(SharedData.GetAnalysisSession());

		const FRenderGraphProvider::TGraphTimeline& GraphTimeline = RenderGraphProvider->GetGraphTimeline();
		GraphTimeline.EnumerateEvents(Viewport.GetStartTime(), Viewport.GetEndTime(),
			[&](double GraphStartTime, double GraphEndTime, uint32, const TSharedPtr<FGraphPacket>& Graph)
		{
			FVisibleGraph& VisibleGraph = AddVisibleGraph(Viewport, *Graph, kBuilderColor);
			VisibleGraph.Draw(Builder);

			if (Viewport.GetViewportDXForDuration(GraphEndTime - GraphStartTime) <= kMinGraphPixels)
			{
				return TraceServices::EEventEnumerate::Continue;
			}

			uint32 DepthOffset;
			double SinglePixelTimeMargin;
			DrawPasses(GraphStartTime, GraphEndTime, VisibleGraph, SinglePixelTimeMargin, DepthOffset);

			Resources.Reset();
			Resources.Reserve(Graph->Textures.Num() + Graph->Buffers.Num());
			TextureIndexToDepth.SetNum(Graph->Textures.Num());
			BufferIndexToDepth.SetNum(Graph->Buffers.Num());

			TBitArray<TInlineAllocator<8>> CulledTextures(true, Graph->Textures.Num());

			const auto IsPacketCulled = [&](const FResourcePacket& Packet)
			{
				if (Packet.bCulled || Packet.bTrackingSkipped || Packet.bTransientUntracked)
				{
					return true;
				}

				if (Packet.bTransient && !ShowTransient())
				{
					return true;
				}

				if (Packet.bExternal && !ShowExternal())
				{
					return true;
				}

				if (Packet.bExtracted && !ShowExtracted())
				{
					return true;
				}

				if (!Packet.bExternal && !Packet.bExtracted && !ShowInternal())
				{
					return true;
				}

				if (!Packet.bTransient && !ShowPooled())
				{
					return true;
				}

				if (FilterSize > 0.0f && Packet.SizeInBytes < uint64(FilterSize * 1024.0f * 1024.0f))
				{
					return true;
				}

				if (!FilterText.IsEmpty() && !Packet.Name.Contains(FilterText))
				{
					return true;
				}

				return false;
			};

			if (ShowTextures())
			{
				for (uint32 TextureIndex = 0, TextureCount = Graph->Textures.Num(); TextureIndex < TextureCount; ++TextureIndex)
				{
					FTexturePacket& Texture = Graph->Textures[TextureIndex];

					const bool bCulled = IsPacketCulled(Texture);
					CulledTextures[TextureIndex] = bCulled;

					if (bCulled)
					{
						continue;
					}

					FResourceEntry& Entry = Resources.AddDefaulted_GetRef();
					Entry.StartTime = Texture.StartTime;
					Entry.EndTime = Texture.EndTime;
					Entry.SizeInBytes = Texture.SizeInBytes;
					Entry.Index = TextureIndex;
					Entry.Order = Texture.Order;
					Entry.Type = ERDGViewableResourceType::Texture;
					Entry.bHasPreviousOwner = Texture.PrevousOwnerHandle.IsValid();
				}
			}

			TBitArray<TInlineAllocator<8>> CulledBuffers(true, Graph->Buffers.Num());

			if (ShowBuffers())
			{
				for (uint32 BufferIndex = 0, BufferCount = Graph->Buffers.Num(); BufferIndex < BufferCount; ++BufferIndex)
				{
					FBufferPacket& Buffer = Graph->Buffers[BufferIndex];

					const bool bCulled = IsPacketCulled(Buffer);
					CulledBuffers[BufferIndex] = bCulled;

					if (bCulled)
					{
						continue;
					}

					FResourceEntry& Entry = Resources.AddDefaulted_GetRef();
					Entry.StartTime = Buffer.StartTime;
					Entry.EndTime = Buffer.EndTime;
					Entry.SizeInBytes = Buffer.SizeInBytes;
					Entry.Index = BufferIndex;
					Entry.Order = Buffer.Order;
					Entry.Type = ERDGViewableResourceType::Buffer;
					Entry.bHasPreviousOwner = Buffer.PrevousOwnerHandle.IsValid();
				}
			}

			switch (ResourceSort)
			{
			case EResourceSort::LargestSize:
				Resources.StableSort([](const FResourceEntry& LHS, const FResourceEntry& RHS)
				{
					return LHS.SizeInBytes > RHS.SizeInBytes;
				});
				break;
			case EResourceSort::SmallestSize:
				Resources.StableSort([](const FResourceEntry& LHS, const FResourceEntry& RHS)
				{
					return LHS.SizeInBytes < RHS.SizeInBytes;
				});
				break;
			case EResourceSort::StartOfLifetime:
				Resources.StableSort([](const FResourceEntry& LHS, const FResourceEntry& RHS)
				{
					return LHS.StartTime < RHS.StartTime;
				});
				break;
			case EResourceSort::EndOfLifetime:
				Resources.StableSort([](const FResourceEntry& LHS, const FResourceEntry& RHS)
				{
					return LHS.EndTime > RHS.EndTime;
				});
				break;
			default:
				Resources.StableSort([](const FResourceEntry& LHS, const FResourceEntry& RHS)
				{
					return LHS.Order < RHS.Order;
				});
				break;
			}

			uint64 MaxSizeInBytes = 0;

			for (FResourceEntry& Entry : Resources)
			{
				if (!Entry.bHasPreviousOwner)
				{
					auto& Array = Entry.Type == ERDGViewableResourceType::Texture ? TextureIndexToDepth : BufferIndexToDepth;
					Array[Entry.Index] = DepthOffset++;
				}

				MaxSizeInBytes = FMath::Max(MaxSizeInBytes, Entry.SizeInBytes);
			}

			if (ShowTextures())
			{
				for (uint32 TextureIndex = 0, TextureCount = Graph->Textures.Num(); TextureIndex < TextureCount; ++TextureIndex)
				{
					if (CulledTextures[TextureIndex])
					{
						continue;
					}

					FTexturePacket& Texture = Graph->Textures[TextureIndex];

					if (Texture.PrevousOwnerHandle.IsValid())
					{
						TextureIndexToDepth[TextureIndex] = TextureIndexToDepth[Texture.PrevousOwnerHandle.GetIndex()];
					}

					const uint32 Depth = TextureIndexToDepth[TextureIndex];
					const double StartTime = Texture.StartTime + SinglePixelTimeMargin;
					const double EndTime = Texture.EndTime;
					const uint32 Color = GetTextureColor(Texture, MaxSizeInBytes);

					const FVisibleTexture VisibleTexture(Viewport, Texture, Color, StartTime, EndTime, Depth);
					VisibleTexture.Draw(Builder);
					VisibleGraph.AddTexture(VisibleTexture);
				}
			}

			if (ShowBuffers())
			{
				for (uint32 BufferIndex = 0, BufferCount = Graph->Buffers.Num(); BufferIndex < BufferCount; ++BufferIndex)
				{
					if (CulledBuffers[BufferIndex])
					{
						continue;
					}

					FBufferPacket& Buffer = Graph->Buffers[BufferIndex];

					if (Buffer.PrevousOwnerHandle.IsValid())
					{
						BufferIndexToDepth[BufferIndex] = BufferIndexToDepth[Buffer.PrevousOwnerHandle.GetIndex()];
					}

					const uint32 Depth = BufferIndexToDepth[BufferIndex];
					const double StartTime = Buffer.StartTime + SinglePixelTimeMargin;
					const double EndTime = Buffer.EndTime;
					const uint32 Color = GetBufferColor(Buffer, MaxSizeInBytes);

					const FVisibleBuffer VisibleBuffer(Viewport, Buffer, Color, StartTime, EndTime, Depth);
					VisibleBuffer.Draw(Builder);
					VisibleGraph.AddBuffer(VisibleBuffer);
				}
			}

			return TraceServices::EEventEnumerate::Continue;
		});
	}
}

void FRenderGraphTrack::BuildFilteredDrawState(FRenderGraphTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context)
{
	const FPacketFilter* PacketFilter = GetPacketFilter(Context.GetEventFilter());

	if (!PacketFilter)
	{
		return;
	}

	const FGraphPacket& Graph = PacketFilter->GetGraph();
	const FVisibleGraph* VisibleGraph = GetVisibleGraph(Graph);

	if (!VisibleGraph)
	{
		return;
	}

	VisibleGraph->Draw(Builder);

	TSet<const FVisibleItem*> VisibleItems;

	for (const FVisibleScope& VisibleScope : VisibleGraph->Scopes)
	{
		if (!PacketFilter->FilterPacket(VisibleScope.GetPacket()))
		{
			continue;
		}

		VisibleItems.Add(&VisibleScope);
	}

	for (const FVisiblePass& VisiblePass : VisibleGraph->Passes)
	{
		const FPassPacket& Pass = VisiblePass.GetPacket();

		if (!PacketFilter->FilterPacket(Pass))
		{
			continue;
		}

		VisibleItems.Add(&VisiblePass);

		if (ShowTextures())
		{
			for (FRDGTextureHandle TextureHandle : Pass.Textures)
			{
				const FTexturePacket& Texture = *Graph.GetTexture(TextureHandle);

				VisibleGraph->EnumerateVisibleTextures(Texture, [&](const FVisibleTexture& VisibleTexture)
				{
					VisibleItems.Add(&VisibleTexture);
				});
			}
		}

		if (ShowBuffers())
		{
			for (FRDGBufferHandle BufferHandle : Pass.Buffers)
			{
				const FBufferPacket& Buffer = *Graph.GetBuffer(BufferHandle);

				VisibleGraph->EnumerateVisibleBuffers(Buffer, [&](const FVisibleBuffer& VisibleBuffer)
				{
					VisibleItems.Add(&VisibleBuffer);
				});
			}
		}

		for (const FVisibleScope& VisibleScope : VisibleGraph->Scopes)
		{
			const FScopePacket& Scope = VisibleScope.GetPacket();
			if (Intersects(Scope, Pass))
			{
				VisibleItems.Add(&VisibleScope);
			}
		}

		const auto AddFencePassEvent = [&](FRDGPassHandle InFencePassHandle)
		{
			const FPassPacket& GraphicsPass = *Graph.GetPass(InFencePassHandle);
			const FVisiblePass& GraphicsVisiblePass = VisibleGraph->GetVisiblePass(GraphicsPass);
			VisibleItems.Add(&GraphicsVisiblePass);
		};

		if (Pass.GraphicsForkPass.IsValid())
		{
			AddFencePassEvent(Pass.GraphicsForkPass);
		}

		if (Pass.GraphicsJoinPass.IsValid())
		{
			AddFencePassEvent(Pass.GraphicsJoinPass);
		}
	}

	const FTimingViewLayout& Layout = Context.GetViewport().GetLayout();

	const auto AddResourcePassEvents = [&](const auto& VisibleResource, TArrayView<const FRDGPassHandle> Passes, uint32 AllocationIndex)
	{
		VisibleResource.Draw(Builder);

		if (VisibleResource.AllocationIndex == AllocationIndex)
		{
			for (FRDGPassHandle PassHandle : Passes)
			{
				const FPassPacket& Pass = *Graph.GetPass(PassHandle);
				const FVisiblePass& VisiblePass = VisibleGraph->GetVisiblePass(Pass);

				VisiblePass.Draw(Builder);

				const float DepthY = VisiblePass.DepthY + VisiblePass.DepthH;

				FSplinePrimitive Spline;
				Spline.Start.X = VisiblePass.SlateX + (VisiblePass.SlateW * 0.5f);
				Spline.Start.Y = DepthY;
				Spline.StartDir = FVector2D(0, -1);
				Spline.End = FVector2D(0, VisibleResource.DepthY - DepthY);
				Spline.EndDir = FVector2D(0, 1);
				Spline.Thickness = 1.0f;
				Spline.Tint = FLinearColor(0.8f, 0.8f, 0.8f, 0.7f);
				Builder.AddSpline(Spline);

				for (const FVisibleScope& VisibleScope : VisibleGraph->Scopes)
				{
					const FScopePacket& Scope = VisibleScope.GetPacket();
					if (Intersects(Scope, Pass))
					{
						VisibleItems.Add(&VisibleScope);
					}
				}
			}
		}
	};

	if (ShowTextures())
	{
		for (const FVisibleTexture& VisibleTexture : VisibleGraph->Textures)
		{
			const FTexturePacket& Texture = VisibleTexture.GetPacket();

			if (PacketFilter->FilterPacketExact(Texture))
			{
				AddResourcePassEvents(VisibleTexture, Texture.Passes, PacketFilter->GetAllocationIndex());
			}
		}
	}

	if (ShowBuffers())
	{
		for (const FVisibleBuffer& VisibleBuffer : VisibleGraph->Buffers)
		{
			const FBufferPacket& Buffer = VisibleBuffer.GetPacket();

			if (PacketFilter->FilterPacketExact(Buffer))
			{
				AddResourcePassEvents(VisibleBuffer, Buffer.Passes, PacketFilter->GetAllocationIndex());
			}
		}
	}

	for (const FVisibleItem* VisibleItem : VisibleItems)
	{
		VisibleItem->Draw(Builder);
	}
}

void FRenderGraphTrack::DrawEvent(const ITimingTrackDrawContext& Context, const ITimingEvent& InTimingEvent, EDrawEventMode InDrawMode) const
{
	if (InTimingEvent.CheckTrack(this) && InTimingEvent.Is<FRenderGraphTimingEvent>())
	{
		const FRenderGraphTimingEvent& TrackEvent = InTimingEvent.As<FRenderGraphTimingEvent>();
		const FVisibleItem TrackItem = TrackEvent.GetItem();

		FRenderGraphTrackDrawHelper DrawHelper(Context);
		DrawHelper.DrawTimingEventHighlight(*this, TrackEvent.GetStartTime(), TrackEvent.GetEndTime(), TrackItem.DepthY, TrackItem.DepthH, InDrawMode);
	}
}

const TSharedPtr<const ITimingEvent> FRenderGraphTrack::GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const
{
	TSharedPtr<FRenderGraphTimingEvent> Event;

	const float DepthY = GetDepthFromLaneY(Viewport.GetLayout(), InPosY - GetPosY());

	for (const FVisibleGraph& Graph : VisibleGraphs)
	{
		if (Graph.Intersects(InPosX, DepthY))
		{
			if (const FVisibleItem* Item = Graph.FindItem(InPosX, DepthY))
			{
				const FPacket& Packet = *Item->Packet;

				if (Packet.Is<FScopePacket>())
				{
					Event = MakeShared<FVisibleScopeEvent>(SharedThis(this), *static_cast<const FVisibleScope*>(Item));
				}
				else if (Packet.Is<FPassPacket>())
				{
					Event = MakeShared<FVisiblePassEvent>(SharedThis(this), *static_cast<const FVisiblePass*>(Item));
				}
				else if (ShowTextures() && Packet.Is<FTexturePacket>())
				{
					Event = MakeShared<FVisibleTextureEvent>(SharedThis(this), *static_cast<const FVisibleTexture*>(Item));
				}
				else if (ShowBuffers() && Packet.Is<FBufferPacket>())
				{
					Event = MakeShared<FVisibleBufferEvent>(SharedThis(this), *static_cast<const FVisibleBuffer*>(Item));
				}
			}
		}
	}

	return Event;
}

TSharedPtr<ITimingEventFilter> FRenderGraphTrack::GetFilterByEvent(const TSharedPtr<const ITimingEvent> InTimingEvent) const
{
	if (InTimingEvent.IsValid() && InTimingEvent->Is<FRenderGraphTimingEvent>())
	{
		bool bFilterable = false;
		if (InTimingEvent->As<FRenderGraphTimingEvent>().GetItem().Intersects(MouseSlateX, MouseDepthY, bFilterable) && bFilterable)
		{
			return MakeShared<FPacketFilter>(StaticCastSharedPtr<const FRenderGraphTimingEvent>(InTimingEvent));
		}
	}
	return nullptr;
}

void FRenderGraphTrack::BuildContextMenu(FMenuBuilder& MenuBuilder)
{
	Super::BuildContextMenu(MenuBuilder);

	UE::Insights::Timing::ITimingViewSession* TimingViewSession = SharedData.GetTimingViewSession();

	MenuBuilder.BeginSection("Visualizer", LOCTEXT("Visualizer", "Visualizer Mode"));
	{
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("Resources", "Resources"),
			LOCTEXT("Resources_Tooltip", "Show each resource on its own line."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, TimingViewSession]()
				{
					Visualizer = EVisualizer::Resources;
					TimingViewSession->ResetSelectedEvent();
					TimingViewSession->ResetEventFilter();
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, TimingViewSession]() { return Visualizer == EVisualizer::Resources; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("TransientHeap", "Transient Heaps"),
			LOCTEXT("TransientHeap_Tooltip", "Shows resource placement in transient heaps."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, TimingViewSession]()
				{
					Visualizer = EVisualizer::TransientMemory;
					TimingViewSession->ResetSelectedEvent();
					TimingViewSession->ResetEventFilter();
					ResourceShow = EResourceShow::All;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return Visualizer == EVisualizer::TransientMemory; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
	}
	MenuBuilder.EndSection();
	
	MenuBuilder.BeginSection("Color", LOCTEXT("ColorMenuHeader", "Track Resource Coloration"));

	MenuBuilder.AddMenuEntry
	(
		LOCTEXT("ColorName", "By Name"),
		LOCTEXT("ColorName_Tooltip", "Each resource name has a unique color."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				ResourceColor = EResourceColor::Name;
				SetDirtyFlag();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return ResourceColor == EResourceColor::Name; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry
	(
		LOCTEXT("ColorType", "By Type"),
		LOCTEXT("ColorType_Tooltip", "Each type of resource has a unique color."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				ResourceColor = EResourceColor::Type;
				SetDirtyFlag();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return ResourceColor == EResourceColor::Type; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry
	(
		LOCTEXT("ColorSize", "By Size"),
		LOCTEXT("ColorSize_Tooltip", "Larger resources are more brightly colored."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				ResourceColor = EResourceColor::Size;
				SetDirtyFlag();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return ResourceColor == EResourceColor::Size; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.AddMenuEntry
	(
		LOCTEXT("ColorTransientCache", "By Transient Cache"),
		LOCTEXT("ColorTransientCache_Tooltip", "Resources that cause a cache miss are brightly colored."),
		FSlateIcon(),
		FUIAction(
			FExecuteAction::CreateLambda([this]()
			{
				ResourceColor = EResourceColor::TransientCache;
				SetDirtyFlag();
			}),
			FCanExecuteAction(),
			FIsActionChecked::CreateLambda([this]() { return ResourceColor == EResourceColor::TransientCache; })
		),
		NAME_None,
		EUserInterfaceActionType::RadioButton
	);

	MenuBuilder.EndSection();

	if (Visualizer == EVisualizer::Resources)
	{
		MenuBuilder.BeginSection("Show", LOCTEXT("ShowMenuHeader", "Track Show Flags"));

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ShowTextures", "Show Textures"),
			LOCTEXT("ShowTextures_Tooltip", "Show Texture resources in the lifetime view."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceShow ^= EResourceShow::Textures;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return EnumHasAnyFlags(ResourceShow, EResourceShow::Textures); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ShowBuffers", "Show Buffers"),
			LOCTEXT("ShowBuffers_Tooltip", "Show Buffer resources in the lifetime view."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceShow ^= EResourceShow::Buffers;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return EnumHasAnyFlags(ResourceShow, EResourceShow::Buffers); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ShowTransient", "Show Transient"),
			LOCTEXT("ShowTransient_Tooltip", "Show transient resources in the lifetime view."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceShow ^= EResourceShow::Transient;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return EnumHasAnyFlags(ResourceShow, EResourceShow::Transient); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ShowExternal", "Show External"),
			LOCTEXT("ShowExternal_Tooltip", "Show external resources in the lifetime view."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceShow ^= EResourceShow::External;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return EnumHasAnyFlags(ResourceShow, EResourceShow::External); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ShowInternal", "Show Internal"),
			LOCTEXT("ShowInternal_Tooltip", "Show internal resources in the lifetime view."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceShow ^= EResourceShow::Internal;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return EnumHasAnyFlags(ResourceShow, EResourceShow::Internal); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ShowExtracted", "Show Extracted"),
			LOCTEXT("ShowExtracted_Tooltip", "Show extracted resources in the lifetime view."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceShow ^= EResourceShow::Extracted;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return EnumHasAnyFlags(ResourceShow, EResourceShow::Extracted); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ShowPooled", "Show Pooled"),
			LOCTEXT("ShowPooled_Tooltip", "Show pooled resources in the lifetime view."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceShow ^= EResourceShow::Pooled;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return EnumHasAnyFlags(ResourceShow, EResourceShow::Pooled); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
		
		MenuBuilder.EndSection();
		MenuBuilder.BeginSection("Sort", LOCTEXT("SortMenuHeader", "Track Sort By"));

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("SortCreation", "Creation"),
			LOCTEXT("SortCreation_Tooltip", "Resources created earlier in the graph builder are ordered first."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceSort = EResourceSort::Creation;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return ResourceSort == EResourceSort::Creation; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("SortLargestSize", "Largest Size"),
			LOCTEXT("SortLargestSize_Tooltip", "Resources with larger allocations are ordered first."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceSort = EResourceSort::LargestSize;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return ResourceSort == EResourceSort::LargestSize; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("SortSmallestSize", "Smallest Size"),
			LOCTEXT("SortSmallestSize_Tooltip", "Resources with smaller allocations are ordered first."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceSort = EResourceSort::SmallestSize;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return ResourceSort == EResourceSort::SmallestSize; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("SortStartOfLifetime", "Start Of Lifetime"),
			LOCTEXT("SortStartOfLifetime_Tooltip", "Resources with earlier starting lifetimes are ordered first."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceSort = EResourceSort::StartOfLifetime;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return ResourceSort == EResourceSort::StartOfLifetime; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("SortEndOfLifetime", "End Of Lifetime"),
			LOCTEXT("SortEndOfLifetime_Tooltip", "Resources with later ending lifetimes are ordered first."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceSort = EResourceSort::EndOfLifetime;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return ResourceSort == EResourceSort::EndOfLifetime; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.EndSection();
	}
	else if (Visualizer == EVisualizer::TransientMemory)
	{
		const uint64 KiB = 1024;
		const uint64 MiB = 1024 * KiB;

		MenuBuilder.BeginSection("HeapZoom", LOCTEXT("HeapZoomHeader", "Vertical Zoom"));
		
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("4MiB", "4MiB"),
			LOCTEXT("4MiB_Tooltip", "Each depth slot in the timing view is 4MiB in size on the vertical axis."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, MiB]()
				{
					TransientBytesPerDepthSlot = 4 * MiB;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, MiB]() { return TransientBytesPerDepthSlot == 4 * MiB; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("2MiB", "2MiB"),
			LOCTEXT("2MiB_Tooltip", "Each depth slot in the timing view is 2MiB in size on the vertical axis."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, MiB]()
				{
					TransientBytesPerDepthSlot = 2 * MiB;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, MiB]() { return TransientBytesPerDepthSlot == 2 * MiB; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("1MiB", "1MiB"),
			LOCTEXT("1MiB_Tooltip", "Each depth slot in the timing view is 1MiB in size on the vertical axis."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, MiB]()
				{
					TransientBytesPerDepthSlot = MiB;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, MiB]() { return TransientBytesPerDepthSlot == MiB; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("512KiB", "512KiB"),
			LOCTEXT("512KiB_Tooltip", "Each depth slot in the timing view is 512KiB in size on the vertical axis."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, KiB]()
				{
					TransientBytesPerDepthSlot = 512 * KiB;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, KiB]() { return TransientBytesPerDepthSlot == 512 * KiB; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("256KiB", "256KiB"),
			LOCTEXT("256KiB_Tooltip", "Each depth slot in the timing view is 256KiB in size on the vertical axis."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, KiB]()
				{
					TransientBytesPerDepthSlot = 256 * KiB;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, KiB]() { return TransientBytesPerDepthSlot == 256 * KiB; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);
		
		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("128KiB", "128KiB"),
			LOCTEXT("128KiB_Tooltip", "Each depth slot in the timing view is 128KiB in size on the vertical axis."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, KiB]()
				{
					TransientBytesPerDepthSlot = 128 * KiB;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, KiB]() { return TransientBytesPerDepthSlot == 128 * KiB; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("64KiB", "64KiB"),
			LOCTEXT("64KiB_Tooltip", "Each depth slot in the timing view is 64KiB in size on the vertical axis."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this, KiB]()
				{
					TransientBytesPerDepthSlot = 64 * KiB;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this, KiB]() { return TransientBytesPerDepthSlot == 64 * KiB; })
			),
			NAME_None,
			EUserInterfaceActionType::RadioButton
		);

		MenuBuilder.EndSection();

		MenuBuilder.BeginSection("Show", LOCTEXT("ShowMenuHeader", "Track Show Flags"));

		MenuBuilder.AddMenuEntry
		(
			LOCTEXT("ShowTracked", "Show Tracked"),
			LOCTEXT("ShowTracked_Tooltip", "Show Tracked resources in the lifetime view."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([this]()
				{
					ResourceShow ^= EResourceShow::Tracked;
					SetDirtyFlag();
				}),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([this]() { return EnumHasAnyFlags(ResourceShow, EResourceShow::Tracked); })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);

		MenuBuilder.EndSection();
	}
	
	MenuBuilder.BeginSection("FilterText", LOCTEXT("FilterTextHeader", "Track Resource Filter"));

	MenuBuilder.AddWidget(
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f)
		[
			// Search box allows for filtering
			SNew(SSearchBox)
			.InitialText(FText::FromString(FilterText))
			.HintText(LOCTEXT("SearchHint", "Filter By Name"))
			.OnTextChanged_Lambda([this](const FText& InText) { FilterText = InText.ToString(); SetDirtyFlag(); })
		]
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(5.f)
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SizeThreshold", "Filter By Size (MB)"))
			]
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SSpinBox<float>)
				.MinValue(0)
				.MaxValue(1024)
				.Value(FilterSize)
				.MaxFractionalDigits(3)
				.MinDesiredWidth(60)
				.OnValueCommitted_Lambda([this](float InValue, ETextCommit::Type) { FilterSize = InValue; SetDirtyFlag(); })
			]
		],
		FText::GetEmpty(),
		true
	);
	
	MenuBuilder.EndSection();
}

void FRenderGraphTrack::InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& InTooltipEvent) const
{
	Tooltip.ResetContent();

	if (InTooltipEvent.CheckTrack(this) && InTooltipEvent.Is<FRenderGraphTimingEvent>())
	{
		const auto AddCommonResourceText = [&](const FResourcePacket& Resource)
		{
			if (Resource.bTransient)
			{
				Tooltip.AddTextLine(TEXT("Transient"), FLinearColor::Red);
				Tooltip.AddNameValueTextLine(TEXT("Transient Cache Hit:"), Resource.bTransientCacheHit ? TEXT("Yes") : TEXT("No"));

				if (Resource.TransientAcquirePass.IsValid())
				{
					Tooltip.AddNameValueTextLine(TEXT("Transient Acquire Pass:"), FString::Printf(TEXT("%u"), Resource.TransientAcquirePass.GetIndex()));
				}
				
				if (Resource.TransientDiscardPass.IsValid())
				{
					Tooltip.AddNameValueTextLine(TEXT("Transient Discard Pass:"), FString::Printf(TEXT("%u"), Resource.TransientDiscardPass.GetIndex()));
				}
			}

			if (!Resource.TransientAllocations.IsEmpty())
			{
				for (const FRHITransientAllocationStats::FAllocation& Allocation : Resource.TransientAllocations)
				{
					Tooltip.AddNameValueTextLine(TEXT("Transient Memory:"),
						FString::Printf(TEXT("Range: %u, Start: %.3fMb, End: %.3fMb"),
							Allocation.MemoryRangeIndex,
							(float)Allocation.OffsetMin / (1024.0f * 1024.0f),
							(float)Allocation.OffsetMax / (1024.0f * 1024.0f)
						)
					);
				}
			}

			if (Resource.bExtracted)
			{
				Tooltip.AddTextLine(TEXT("Extracted"), FLinearColor::Red);
			}

			if (Resource.bExternal)
			{
				Tooltip.AddTextLine(TEXT("External"), FLinearColor::Red);
			}

			if (Resource.bTransientUntracked)
			{
				Tooltip.AddTextLine(TEXT("Untracked by RDG"), FLinearColor::Red);
			}
		};

		if (InTooltipEvent.Is<FVisibleScopeEvent>())
		{
			const auto& TooltipEvent = InTooltipEvent.As<FVisibleScopeEvent>();
			const FScopePacket& Scope = TooltipEvent.GetPacket();
			Tooltip.AddTitle(Scope.Name);

			const uint32 PassCount = Scope.LastPass.GetIndex() - Scope.FirstPass.GetIndex() + 1;
			Tooltip.AddNameValueTextLine(TEXT("Passes:"), FString::Printf(TEXT("%d"), PassCount));
		}
		else if (InTooltipEvent.Is<FVisiblePassEvent>())
		{
			const auto& TooltipEvent = InTooltipEvent.As<FVisiblePassEvent>();
			const FPassPacket& Pass = TooltipEvent.GetPacket();

			Tooltip.AddTitle(GetSanitizedName(Pass.Name));
			Tooltip.AddNameValueTextLine(TEXT("Handle:"), FString::Printf(TEXT("%d"), Pass.Handle.GetIndex()));

			if (Pass.bCulled)
			{
				Tooltip.AddTextLine(TEXT("Culled"), FLinearColor::Red);
			}
			else
			{
				Tooltip.AddNameValueTextLine(TEXT("Used Textures:"), FString::Printf(TEXT("%d"), Pass.Textures.Num()));
				Tooltip.AddNameValueTextLine(TEXT("Used Buffers:"), FString::Printf(TEXT("%d"), Pass.Buffers.Num()));
			}

			if (Pass.bParallelExecuteAllowed)
			{
				Tooltip.AddNameValueTextLine(TEXT("Parallel Execute:"), Pass.bParallelExecute
					? Pass.bParallelExecuteAsyncAllowed
						? TEXT("Yes (Async)")
						: TEXT("Yes (Await)")
					: TEXT("No"));
			}
			else
			{
				Tooltip.AddNameValueTextLine(TEXT("Parallel Execute:"), TEXT("Not Allowed"));
			}

			if (Pass.bSkipRenderPassBegin || Pass.bSkipRenderPassEnd)
			{
				Tooltip.AddTextLine(TEXT("Merged RenderPass"), FLinearColor::Red);
			}

			if (Pass.GraphicsForkPass.IsValid())
			{
				Tooltip.AddNameValueTextLine(TEXT("GraphicsForkPass:"), FString::Printf(TEXT("%d"), Pass.GraphicsForkPass.GetIndex()));
			}
			
			if (Pass.GraphicsJoinPass.IsValid())
			{
				Tooltip.AddNameValueTextLine(TEXT("GraphicsJoinPass:"), FString::Printf(TEXT("%d"), Pass.GraphicsJoinPass.GetIndex()));
			}
		}
		else if (InTooltipEvent.Is<FVisibleTextureEvent>())
		{
			const auto& TooltipEvent = InTooltipEvent.As<FVisibleTextureEvent>();
			const FTexturePacket& Texture = TooltipEvent.GetPacket();

			Tooltip.AddTitle(Texture.Name);
			Tooltip.AddNameValueTextLine(TEXT("Dimension:"), GetTextureDimensionString(Texture.Desc.Dimension));
			Tooltip.AddNameValueTextLine(TEXT("Create Flags:"), GetTextureCreateFlagsName(Texture.Desc.Flags));
			Tooltip.AddNameValueTextLine(TEXT("Format:"), UEnum::GetValueAsString(Texture.Desc.Format));
			Tooltip.AddNameValueTextLine(TEXT("Extent:"), FString::Printf(TEXT("%d, %d"), Texture.Desc.Extent.X, Texture.Desc.Extent.Y));
			Tooltip.AddNameValueTextLine(TEXT("Depth:"), FString::Printf(TEXT("%d"), Texture.Desc.Depth));
			Tooltip.AddNameValueTextLine(TEXT("Mips:"), FString::Printf(TEXT("%d"), Texture.Desc.NumMips));
			Tooltip.AddNameValueTextLine(TEXT("Array Size:"), FString::Printf(TEXT("%d"), Texture.Desc.ArraySize));
			Tooltip.AddNameValueTextLine(TEXT("Samples:"), FString::Printf(TEXT("%d"), Texture.Desc.NumSamples));
			Tooltip.AddNameValueTextLine(TEXT("Used Passes:"), FString::Printf(TEXT("%d"), Texture.Passes.Num()));
			AddCommonResourceText(Texture);
		}
		else if (InTooltipEvent.Is<FVisibleBufferEvent>())
		{
			const auto& TooltipEvent = InTooltipEvent.As<FVisibleBufferEvent>();
			const FBufferPacket& Buffer = TooltipEvent.GetPacket();

			Tooltip.AddTitle(Buffer.Name);
			Tooltip.AddNameValueTextLine(TEXT("Usage Flags:"), GetBufferUsageFlagsName(Buffer.Desc.Usage));
			Tooltip.AddNameValueTextLine(TEXT("Bytes Per Element:"), FString::Printf(TEXT("%d"), Buffer.Desc.BytesPerElement));
			Tooltip.AddNameValueTextLine(TEXT("Elements:"), FString::Printf(TEXT("%d"), Buffer.Desc.NumElements));
			Tooltip.AddNameValueTextLine(TEXT("Used Passes:"), FString::Printf(TEXT("%d"), Buffer.Passes.Num()));
			AddCommonResourceText(Buffer);
		}

		Tooltip.UpdateLayout();
	}
}

}} //namespace UE::RenderGraphInsights

#undef LOCTEXT_NAMESPACE
