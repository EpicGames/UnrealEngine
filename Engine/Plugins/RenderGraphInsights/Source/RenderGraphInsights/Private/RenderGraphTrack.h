// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ViewModels/TimingTrackViewport.h"
#include "Insights/ViewModels/TooltipDrawState.h"
#include "RenderGraphProvider.h"

namespace UE::RenderGraphInsights { class FRenderGraphTrackDrawStateBuilder; }
struct FTimingViewLayout;

class FMenuBuilder;
class ITimingTrackDrawContext;
class ITimingTrackUpdateContext;
class FTimingTrackViewport;

namespace UE { namespace RenderGraphInsights {

class FRenderGraphTimingViewSession;
class FPacketFilter;

class FBoundingBox
{
public:
	FBoundingBox() = default;

	FBoundingBox(FVector2D InMin, FVector2D InMax)
		: Min(InMin)
		, Max(InMax)
	{}

	FVector2D Min = FVector2D::ZeroVector;
	FVector2D Max = FVector2D::ZeroVector;
};

class FVisibleItem
{
public:
	FVisibleItem(const FTimingTrackViewport& InViewport, const FPacket& InPacket, uint32 InColor, double InStartTime, double InEndTime, float InDepthY, float InDepthH = 1.0f)
		: Packet(&InPacket)
		, Name(*InPacket.Name)
		, StartTime(InStartTime)
		, EndTime(InEndTime)
		, DepthY(InDepthY)
		, DepthH(InDepthH)
		, Color(InColor)
	{
		SlateX = InViewport.TimeToSlateUnitsRounded(StartTime);
		SlateW = InViewport.TimeToSlateUnitsRounded(EndTime) - SlateX;
	}

	virtual bool Intersects(float InSlateX, float InDepthY, bool& bOutFilterable) const
	{
		bOutFilterable = true;
		return InSlateX >= SlateX && InSlateX < (SlateX + SlateW) && InDepthY > DepthY && InDepthY < (DepthY + DepthH);
	}

	bool Intersects(float InSlateX, float InDepthY) const
	{
		bool bFilterable = false;
		return Intersects(InSlateX, InDepthY, bFilterable);
	}

	FBoundingBox GetBoundingBox(const FTimingViewLayout& Layout) const;

	virtual void Draw(FRenderGraphTrackDrawStateBuilder& Builder) const;

	const FPacket* Packet{};
	const TCHAR* Name{};
	double StartTime{};
	double EndTime{};
	float DepthY{};
	float DepthH{};
	float SlateX{};
	float SlateW{};
	uint32 Color{};
	uint32 Index{};
};

template <typename InPacketType>
class TVisibleItemHelper : public FVisibleItem
{
public:
	using PacketType = InPacketType;

	TVisibleItemHelper(const FTimingTrackViewport& Viewport, const PacketType& InPacket, uint32 InColor, double InStartTime, double InEndTime, float InDepthY, float InDepthH = 1.0f)
		: FVisibleItem(Viewport, InPacket, InColor, InStartTime, InEndTime, InDepthY, InDepthH)
	{}

	const PacketType& GetPacket() const
	{
		return static_cast<const PacketType&>(*Packet);
	}
};

class FVisiblePass final : public TVisibleItemHelper<FPassPacket>
{
public:
	FVisiblePass(const FTimingTrackViewport& Viewport, const FPassPacket& InPacket, uint32 InColor, double InStartTime, double InEndTime, float InDepthY, float InDepthH = 1.0f)
		: TVisibleItemHelper(Viewport, InPacket, InColor, InStartTime, InEndTime, InDepthY, InDepthH)
	{}

	bool Intersects(float InSlateX, float InDepthY, bool& bOutFilterable) const override
	{
		// For hit-testing, treat the pass as unbounded along Y, so that it's a column.
		if (InSlateX >= SlateX && InSlateX < (SlateX + SlateW) && InDepthY >= DepthY)
		{
			bOutFilterable = InDepthY < (DepthY + DepthH);
			return true;
		}
		return false;
	}

	using FVisibleItem::Intersects;
};

using FVisibleScope   = TVisibleItemHelper<FScopePacket>;

template <typename InPacketType>
class TVisibleResourceHelper : public FVisibleItem
{
public:
	using PacketType = InPacketType;

	TVisibleResourceHelper(const FTimingTrackViewport& Viewport, const PacketType& InPacket, uint32 InColor, double InStartTime, double InEndTime, float InDepthY, float InDepthH = 1.0f, uint32 InAllocationIndex = 0)
		: FVisibleItem(Viewport, InPacket, InColor, InStartTime, InEndTime, InDepthY, InDepthH)
		, AllocationIndex(InAllocationIndex)
	{}

	const PacketType& GetPacket() const
	{
		return static_cast<const PacketType&>(*Packet);
	}

	uint32 AllocationIndex;
};

using FVisibleTexture = TVisibleResourceHelper<FTexturePacket>;
using FVisibleBuffer  = TVisibleResourceHelper<FBufferPacket>;

class FVisibleGraph final : public TVisibleItemHelper<FGraphPacket>
{
public:
	FVisibleGraph(const FTimingTrackViewport& Viewport, const FGraphPacket& InGraph, uint32 InColor, float InDepthH)
		: TVisibleItemHelper(Viewport, InGraph, InColor, InGraph.StartTime, InGraph.EndTime, 0.0f, InDepthH)
	{}

	void AddScope(const FVisibleScope& VisibleScope);
	void AddPass(const FVisiblePass& VisiblePass);
	void AddTexture(const FVisibleTexture& VisibleTexture);
	void AddBuffer(const FVisibleBuffer& VisibleBuffer);

	void Reset();

	const FVisibleItem* FindItem(float InSlateX, float InDepthY) const;

	const FVisibleScope& GetVisibleScope(const FScopePacket& Scope) const
	{
		return Scopes[Scope.VisibleIndex];
	}

	const FVisiblePass& GetVisiblePass(const FPassPacket& Pass) const
	{
		return Passes[Pass.VisibleIndex];
	}

	template <typename FunctionType>
	void EnumerateVisibleTextures(const FTexturePacket& Texture, FunctionType Function) const
	{
		for (int32 VisibleIndex : Texture.VisibleItems)
		{
			Function(Textures[VisibleIndex]);
		}
	}

	template <typename FunctionType>
	void EnumerateVisibleBuffers(const FBufferPacket& Buffer, FunctionType Function) const
	{
		for (int32 VisibleIndex : Buffer.VisibleItems)
		{
			Function(Buffers[VisibleIndex]);
		}
	}

	void Draw(FRenderGraphTrackDrawStateBuilder& Builder) const override;

	bool IsCulled() const
	{
		return Passes.IsEmpty();
	}

	TArray<FVisibleScope> Scopes;
	TArray<FVisiblePass> Passes;
	TArray<FVisibleTexture> Textures;
	TArray<FVisibleBuffer> Buffers;
	TArray<uint32> AsyncComputePasses;
	TArray<float> TransientMemoryRangeDepthOffsets;
	float HeaderEndDepth{};
	float HeaderPassBarDepth{};
};

class FRenderGraphTrack final : public FBaseTimingTrack
{
	INSIGHTS_DECLARE_RTTI(FRenderGraphTrack, FBaseTimingTrack)
	using Super = FBaseTimingTrack;

public:
	FRenderGraphTrack(const FRenderGraphTimingViewSession& InSharedData);

	//~ Begin FBaseTimingTrack interface
	void Reset() override;
	void PreUpdate(const ITimingTrackUpdateContext& Context) override;
	void Update(const ITimingTrackUpdateContext& Context) override;
	void PostUpdate(const ITimingTrackUpdateContext& Context) override;
	void Draw(const ITimingTrackDrawContext& Context) const override;
	void PostDraw(const ITimingTrackDrawContext& Context) const override;
	void DrawEvent(const ITimingTrackDrawContext& Context, const ITimingEvent& InTimingEvent, EDrawEventMode InDrawMode) const override;
	const TSharedPtr<const ITimingEvent> GetEvent(float InPosX, float InPosY, const FTimingTrackViewport& Viewport) const override;
	TSharedPtr<ITimingEventFilter> GetFilterByEvent(const TSharedPtr<const ITimingEvent> InTimingEvent) const;
	void BuildContextMenu(FMenuBuilder& MenuBuilder) override;
	void InitTooltip(FTooltipDrawState& Tooltip, const ITimingEvent& HoveredTimingEvent) const override;
	//~ End FBaseTimingTrack interface

private:
	const FRenderGraphTimingViewSession& SharedData;

	void BuildDrawState(FRenderGraphTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context);
	void BuildFilteredDrawState(FRenderGraphTrackDrawStateBuilder& Builder, const ITimingTrackUpdateContext& Context);

	FVisibleGraph& AddVisibleGraph(const FTimingTrackViewport& Viewport, const FGraphPacket& Graph, uint32 Color)
	{
		const uint32 VisibleIndex = VisibleGraphs.Num();
		FVisibleGraph& VisibleGraph = VisibleGraphs.Emplace_GetRef(Viewport, Graph, Color, 1.0f);
		check(VisibleGraph.GetPacket().VisibleIndex == kInvalidVisibleIndex);
		VisibleGraph.GetPacket().VisibleIndex = VisibleIndex;
		VisibleGraph.Index = VisibleIndex;

		for (uint64 ByteOffset : Graph.TransientMemoryRangeByteOffsets)
		{
			VisibleGraph.TransientMemoryRangeDepthOffsets.Emplace(FMath::CeilToFloat(TransientByteOffsetToDepth(ByteOffset)));
		}

		return VisibleGraph;
	}

	const FVisibleGraph* GetVisibleGraph(const FGraphPacket& Packet) const
	{
		if (Packet.VisibleIndex != kInvalidVisibleIndex)
		{
			return &VisibleGraphs[Packet.VisibleIndex];
		}
		return nullptr;
	}

	float TransientByteOffsetToDepth(uint64 ByteOffset) const;
	float TransientByteOffsetToDepthAligned(uint64 ByteOffset) const;
	float TransientByteOffsetToSlateY(const FTimingViewLayout& Layout, uint64 ByteOffset) const;

	float MouseSlateX{};
	float MouseSlateY{};
	float MouseDepthY{};

	TArray<FVisibleGraph> VisibleGraphs;

	struct FSpline
	{
		float Thickness{};
		FVector2D Start = FVector2D::ZeroVector;
		FVector2D StartDir = FVector2D::ZeroVector;
		FVector2D End = FVector2D::ZeroVector;
		FVector2D EndDir = FVector2D::ZeroVector;
		FLinearColor Tint = FLinearColor::White;
	};

	TArray<FSpline> Splines;

	uint32 GetTextureColor(const FTexturePacket& Texture, uint64 MaxSizeInBytes) const;
	uint32 GetBufferColor(const FBufferPacket& Buffer, uint64 MaxSizeInBytes) const;

	bool ShowTextures() const	{ return EnumHasAnyFlags(ResourceShow, EResourceShow::Textures); }
	bool ShowBuffers() const	{ return EnumHasAnyFlags(ResourceShow, EResourceShow::Buffers); }
	bool ShowTransient() const	{ return EnumHasAnyFlags(ResourceShow, EResourceShow::Transient); }
	bool ShowExternal() const	{ return EnumHasAnyFlags(ResourceShow, EResourceShow::External); }
	bool ShowInternal() const	{ return EnumHasAnyFlags(ResourceShow, EResourceShow::Internal); }
	bool ShowExtracted() const	{ return EnumHasAnyFlags(ResourceShow, EResourceShow::Extracted); }
	bool ShowPooled() const		{ return EnumHasAnyFlags(ResourceShow, EResourceShow::Pooled); }
	bool ShowTracked() const	{ return EnumHasAnyFlags(ResourceShow, EResourceShow::Tracked); }

	enum class EResourceShow
	{
		Textures		= 1 << 0,
		Buffers			= 1 << 1,
		Transient		= 1 << 2,
		External		= 1 << 3,
		Internal		= 1 << 4,
		Extracted		= 1 << 5,
		Pooled			= 1 << 6,
		Tracked			= 1 << 7,
		All				= Textures | Buffers | Transient | External | Internal | Extracted | Pooled | Tracked
	};
	FRIEND_ENUM_CLASS_FLAGS(FRenderGraphTrack::EResourceShow);

	enum class EResourceSort
	{
		Creation,
		LargestSize,
		SmallestSize,
		StartOfLifetime,
		EndOfLifetime
	};

	enum class EResourceColor
	{
		Name,
		Type,
		Size,
		TransientCache
	};

	enum class EVisualizer
	{
		Resources,
		TransientMemory
	};

	EResourceShow ResourceShow = EResourceShow::All;
	EResourceSort ResourceSort = EResourceSort::Creation;
	EResourceColor ResourceColor = EResourceColor::Name;
	EVisualizer Visualizer = EVisualizer::Resources;
	FString FilterText;
	float FilterSize{};
	uint64 TransientBytesPerDepthSlot = 1024 * 1024;

	FTooltipDrawState SelectedTooltipState;

	int32 NumLanes;
	TSharedRef<struct FRenderGraphTrackDrawState> DrawState;
	TSharedRef<struct FRenderGraphTrackDrawState> FilteredDrawState;

	struct FFilteredDrawStateInfo
	{
		double ViewportStartTime = 0.0;
		double ViewportScaleX = 0.0;
		double LastBuildDuration = 0.0;
		TWeakPtr<ITimingEventFilter> LastEventFilter;
		uint32 Counter = 0;
		mutable float Opacity = 0.0f;
	};
	FFilteredDrawStateInfo FilteredDrawStateInfo;
};

ENUM_CLASS_FLAGS(FRenderGraphTrack::EResourceShow);

}} //namespace UE::RenderGraphInsights
