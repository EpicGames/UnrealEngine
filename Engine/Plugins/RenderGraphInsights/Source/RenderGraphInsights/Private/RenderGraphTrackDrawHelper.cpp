// Copyright Epic Games, Inc. All Rights Reserved.

#include "RenderGraphTrackDrawHelper.h"

#include "Brushes/SlateRoundedBoxBrush.h"
#include "Insights/ViewModels/ITimingViewDrawHelper.h"
#include "Styling/CoreStyle.h"

// TraceInsightsCore
#include "InsightsCore/Common/PaintUtils.h"
#include "InsightsCore/Common/TimeUtils.h"

// TraceInsights
#include "Insights/ViewModels/BaseTimingTrack.h"
#include "Insights/ViewModels/TimingEvent.h"
#include "Insights/ViewModels/TimingTrackViewport.h"

namespace UE { namespace RenderGraphInsights {

constexpr float kBorderColorFactor = 1.25f;

inline FLinearColor ConvertToLinearColor(uint32 InColor)
{
	FLinearColor LinearColor;
	LinearColor.R = ((InColor >> 16) & 0xFF) / 255.0f;
	LinearColor.G = ((InColor >> 8) & 0xFF) / 255.0f;
	LinearColor.B = ((InColor) & 0xFF) / 255.0f;
	LinearColor.A = ((InColor >> 24) & 0xFF) / 255.0f;
	return LinearColor;
}

FRenderGraphTrackDrawStateBuilder::FRenderGraphTrackDrawStateBuilder(FRenderGraphTrackDrawState& InState, const FTimingTrackViewport& InViewport)
	: DrawState(InState)
	, Viewport(InViewport)
	, EventFont(FCoreStyle::GetDefaultFontStyle("Regular", 8))
{
	DrawState.Reset();
}

void FRenderGraphTrackDrawStateBuilder::AppendDurationToEventName(FString& InOutEventName, const double InDuration)
{
	InOutEventName += TEXT(" (");
	InOutEventName += UE::Insights::FormatTimeAuto(InDuration);
	InOutEventName += TEXT(")");
}

void FRenderGraphTrackDrawStateBuilder::AddEvent(double EventStartTime, double EventEndTime, uint32 EventDepth, const TCHAR* EventName, uint64 EventType, uint32 EventColor)
{
	if (EventColor == 0)
	{
		EventColor = FTimingEvent::ComputeEventColor(EventName);
	}

	AddEvent(EventStartTime, EventEndTime, EventDepth, EventColor,
		[EventName, EventType, EventStartTime, EventEndTime](float Width)
	{
		FString Name = EventName;

		if (Width > Name.Len() * 4.0f + 32.0f)
		{
			AppendDurationToEventName(Name, EventEndTime - EventStartTime);
		}

		return Name;
	});
}

void FRenderGraphTrackDrawStateBuilder::AddEvent(double EventStartTime, double EventEndTime, uint32 EventDepth, uint32 EventColor, GetEventNameCallback GetEventNameCallback)
{
	float EventX1 = Viewport.TimeToSlateUnitsRounded(EventStartTime);
	if (EventX1 > Viewport.GetWidth())
	{
		return;
	}

	const double RestrictedEndTime = Viewport.RestrictEndTime(EventEndTime);
	float EventX2 = Viewport.TimeToSlateUnitsRounded(RestrictedEndTime);
	if (EventX2 < 0)
	{
		return;
	}

	// Timing events are displayed with minimum 1px (including empty ones).
	if (EventX1 == EventX2)
	{
		EventX2 = EventX1 + 1.0f;
	}

	const int32 Depth = static_cast<int32>(EventDepth);
	MaxDepthCached = FMath::Max(MaxDepthCached, Depth);

	// Ensure we have enough slots in array. See LastBox[Depth] usage.
	if (LastBox.Num() <= Depth)
	{
		LastBox.AddDefaulted(Depth + 1 - LastBox.Num());
	}

	// Ensure we have enough slots in array. See LastEventX2[Depth] usage.
	while (LastEventX2.Num() <= Depth)
	{
		LastEventX2.Add(-1.0f);
	}

	// Limit event width on the viewport's left side.
	// This also makes the text to be displayed in viewport, for very long events.
	const float MinX = -2.0f; // -2 allows event border to remain outside screen
	if (EventX1 < MinX)
	{
		EventX1 = MinX;
	}

	// Limit event width on the viewport's right side.
	const float MaxX = Viewport.GetWidth() + 2.0f; // +2 allows event border to remain outside screen
	if (EventX2 > MaxX)
	{
		EventX2 = MaxX;
	}

	const float EventW = EventX2 - EventX1;

	// Optimization...
	if (EventW == 1.0f && EventX1 == LastEventX2[Depth] - 1.0f)
	{
		// Do no draw 1 pixel event if the last event was ended on that pixel.
		return;
	}

	const FLinearColor EventColorFill = ConvertToLinearColor(EventColor);

	// Save X2, for current depth.
	LastEventX2[Depth] = EventX2;

	if (EventW > 2.0f)
	{
		FBoxData& Box = LastBox[Depth];
		if (Box.X1 < Box.X2)
		{
			FlushBox(Box, Depth);
			Box.Reset();
		}

		// Fill inside of the timing event box.
		DrawState.InsideBoxes.AddUninitialized();
		FRenderGraphTrackDrawState::FBoxPrimitive& InsideBox = DrawState.InsideBoxes.Last();
		InsideBox.DepthY = Depth;
		InsideBox.DepthH = 1.0f;
		InsideBox.X = EventX1 + 1.0f;
		InsideBox.W = EventW - 2.0f;
		InsideBox.Color = EventColorFill;

		// Add border around the timing event box.
		DrawState.Borders.AddUninitialized();
		FRenderGraphTrackDrawState::FBoxPrimitive& BorderBox = DrawState.Borders.Last();
		BorderBox.DepthY = Depth;
		BorderBox.DepthH = 1.0f;
		BorderBox.X = EventX1;
		BorderBox.W = EventW;
		BorderBox.Color = FLinearColor(EventColorFill.R * kBorderColorFactor, EventColorFill.G * kBorderColorFactor, EventColorFill.B * kBorderColorFactor, EventColorFill.A);
	}
	else // 1px or 2px boxes
	{
		FBoxData& Box = LastBox[Depth];

		// Check if we can merge this box with previous one, if any.
		// Note: We are assuming events are processed in sorted order by X1.
		if (EventColor == Box.Color && // same color
			EventX1 <= Box.X2) // overlapping or adjacent
		{
			// Merge it with previous box.
			Box.X2 = EventX2;
		}
		else
		{
			// Flush previous box, if any.
			if (Box.X1 < Box.X2)
			{
				FlushBox(Box, Depth);
			}

			// Start new "merge box".
			Box.X1 = EventX1;
			Box.X2 = EventX2;
			Box.Color = EventColor;
			Box.LinearColor = FLinearColor(EventColorFill.R * kBorderColorFactor, EventColorFill.G * kBorderColorFactor, EventColorFill.B * kBorderColorFactor, EventColorFill.A);
		}
	}

	if (EventW > 8.0f)
	{
		const FString Name = GetEventNameCallback(EventW);

		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const int32 LastWholeCharacterIndex = FontMeasureService->FindLastWholeCharacterIndexBeforeOffset(Name, EventFont, FMath::RoundToInt(EventW - 2.0f));

		if (LastWholeCharacterIndex >= 0)
		{
			const bool bIsDarkColor = (EventColorFill.GetLuminance() < 0.4f);

			DrawState.Texts.AddDefaulted();
			FRenderGraphTrackDrawState::FTextPrimitive& DrawText = DrawState.Texts.Last();
			DrawText.Depth = Depth;
			DrawText.X = EventX1 + 2.0f;
			DrawText.Text = Name.Left(LastWholeCharacterIndex + 1);
			DrawText.bWhite = bIsDarkColor;
			DrawText.Color = FLinearColor(EventColorFill.R * kBorderColorFactor, EventColorFill.G * kBorderColorFactor, EventColorFill.B * kBorderColorFactor, EventColorFill.A);
		}
	}
}

void FRenderGraphTrackDrawStateBuilder::AddEvent(double EventStartTime, double EventEndTime, float EventDepthY, float EventDepthH, uint32 EventColor, GetEventNameCallback GetEventNameCallback)
{
	float EventX1 = Viewport.TimeToSlateUnitsRounded(EventStartTime);
	if (EventX1 > Viewport.GetWidth())
	{
		return;
	}

	const double RestrictedEndTime = Viewport.RestrictEndTime(EventEndTime);
	float EventX2 = Viewport.TimeToSlateUnitsRounded(RestrictedEndTime);
	if (EventX2 < 0)
	{
		return;
	}

	// Timing events are displayed with minimum 1px (including empty ones).
	if (EventX1 == EventX2)
	{
		EventX2 = EventX1 + 1.0f;
	}

	MaxDepth = FMath::Max<int32>(MaxDepth, FMath::CeilToFloat(EventDepthY + EventDepthH));

	// Limit event width on the viewport's left side.
	// This also makes the text to be displayed in viewport, for very long events.
	const float MinX = -2.0f; // -2 allows event border to remain outside screen
	if (EventX1 < MinX)
	{
		EventX1 = MinX;
	}

	// Limit event width on the viewport's right side.
	const float MaxX = Viewport.GetWidth() + 2.0f; // +2 allows event border to remain outside screen
	if (EventX2 > MaxX)
	{
		EventX2 = MaxX;
	}

	const float EventW = EventX2 - EventX1;
	const FLinearColor EventColorFill = ConvertToLinearColor(EventColor);

	if (EventW > 0.0f)
	{
		DrawState.Boxes.AddUninitialized();
		FRenderGraphTrackDrawState::FBoxPrimitive& DrawBox = DrawState.Boxes.Last();
		DrawBox.X = EventX1;
		DrawBox.W = EventW;
		DrawBox.DepthY = EventDepthY;
		DrawBox.DepthH = EventDepthH;
		DrawBox.Color = FLinearColor(EventColorFill.R * kBorderColorFactor, EventColorFill.G * kBorderColorFactor, EventColorFill.B * kBorderColorFactor, EventColorFill.A);

		// Fill inside of the timing event box.
		DrawState.InsideBoxes.AddUninitialized();
		FRenderGraphTrackDrawState::FBoxPrimitive& InsideBox = DrawState.InsideBoxes.Last();
		InsideBox.X = EventX1 + 1.0f;
		InsideBox.W = EventW - 2.0f;
		InsideBox.DepthY = EventDepthY;
		InsideBox.DepthH = EventDepthH;
		InsideBox.Color = EventColorFill;

		// Add border around the timing event box.
		DrawState.Borders.AddUninitialized();
		FRenderGraphTrackDrawState::FBoxPrimitive& BorderBox = DrawState.Borders.Last();
		BorderBox.X = EventX1;
		BorderBox.W = EventW;
		BorderBox.DepthY = EventDepthY;
		BorderBox.DepthH = EventDepthH;
		BorderBox.Color = FLinearColor(EventColorFill.R * kBorderColorFactor, EventColorFill.G * kBorderColorFactor, EventColorFill.B * kBorderColorFactor, EventColorFill.A);
	}

	// Draw the name of the timing event.
	if (EventW > 8.0f && EventDepthH >= 1.0f)
	{
		const FString Name = GetEventNameCallback(EventW);

		const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		const int32 LastWholeCharacterIndex = FontMeasureService->FindLastWholeCharacterIndexBeforeOffset(Name, EventFont, FMath::RoundToInt(EventW - 2.0f));

		if (LastWholeCharacterIndex >= 0)
		{
			// Grey threshold is shifted toward black (0.4 instead of 0.5 in test below) due to "area rule":
			// a large gray surface (background of a timing event in this case) is perceived lighter than a smaller area (text pixels).
			// Ref: https://books.google.ro/books?id=0pVr7dhmdWYC
			const bool bIsDarkColor = (EventColorFill.GetLuminance() < 0.4f);

			DrawState.Texts.AddDefaulted();
			FRenderGraphTrackDrawState::FTextPrimitive& DrawText = DrawState.Texts.Last();
			DrawText.X = EventX1 + 2.0f;
			DrawText.Depth = EventDepthY;
			DrawText.Text = Name.Left(LastWholeCharacterIndex + 1);
			DrawText.bWhite = bIsDarkColor;
			DrawText.Color = FLinearColor(EventColorFill.R * kBorderColorFactor, EventColorFill.G * kBorderColorFactor, EventColorFill.B * kBorderColorFactor, EventColorFill.A);
		}
	}
}

void FRenderGraphTrackDrawStateBuilder::FlushBox(const FBoxData& Box, const int32 Depth)
{
	DrawState.Boxes.AddUninitialized();
	FRenderGraphTrackDrawState::FBoxPrimitive& DrawBox = DrawState.Boxes.Last();
	DrawBox.DepthY = Depth;
	DrawBox.DepthH = 1.0f;
	DrawBox.X = Box.X1;
	DrawBox.W = Box.X2 - Box.X1;
	DrawBox.Color = Box.LinearColor;
}

void FRenderGraphTrackDrawStateBuilder::Flush()
{
	for (int32 Depth = 0; Depth <= MaxDepthCached; ++Depth)
	{
		const FBoxData& Box = LastBox[Depth];
		if (Box.X1 < Box.X2)
		{
			FlushBox(Box, Depth);
		}
	}
}

FRenderGraphTrackDrawHelper::FRenderGraphTrackDrawHelper(const ITimingTrackDrawContext& Context)
	: DrawContext(Context.GetDrawContext())
	, Viewport(Context.GetViewport())
	, ParentHelper(Context.GetHelper())
	, WhiteBrush(ParentHelper.GetWhiteBrush())
	, EventBorderBrush(ParentHelper.GetEventBorderBrush())
	, HoveredEventBorderBrush(ParentHelper.GetHoveredEventBorderBrush())
	, SelectedEventBorderBrush(ParentHelper.GetSelectedEventBorderBrush())
	, ValidAreaColor(ParentHelper.GetValidAreaColor())
	, InvalidAreaColor(ParentHelper.GetInvalidAreaColor())
	, EdgeColor(ParentHelper.GetEdgeColor())
	, EventFont(ParentHelper.GetEventFont())
	, ReservedLayerId(ParentHelper.GetFirstLayerId())
{}

void FRenderGraphTrackDrawHelper::DrawEvents(const FRenderGraphTrackDrawState& DrawState, const FBaseTimingTrack& Track) const
{
	const float TrackH = Track.GetHeight();
	if (TrackH > 0.0f)
	{
		const FTimingViewLayout& Layout = Viewport.GetLayout();

		float TopLaneY = Track.GetPosY() + Layout.TimelineDY + 1.0f;

		// Draw filled boxes (merged borders).
		{
			const int32 EventFillLayerId = ReservedLayerId + int32(EDrawLayer::EventFill);

			for (const FRenderGraphTrackDrawState::FBoxPrimitive& Box : DrawState.Boxes)
			{
				const float Y = GetLaneY(TopLaneY, Layout, Box.DepthY);
				const float H = GetLaneHeight(Layout, Box.DepthH);
				DrawContext.DrawBox(EventFillLayerId, Box.X, Y, Box.W, H, WhiteBrush, Box.Color);
			}
		}

		// Draw filled boxes (event inside area).
		if (Layout.EventH > 2.0f)
		{
			const int32 EventFillLayerId = ReservedLayerId + int32(EDrawLayer::EventFill);

			for (const FRenderGraphTrackDrawState::FBoxPrimitive& Box : DrawState.InsideBoxes)
			{
				const float Y = GetLaneY(TopLaneY, Layout, Box.DepthY) + 1.0f;
				const float H = GetLaneHeight(Layout, Box.DepthH) - 2.0f;
				DrawContext.DrawBox(EventFillLayerId, Box.X, Y, Box.W, H, WhiteBrush, Box.Color);
			}
		}

		// Draw borders.
		{
			const int32 EventBorderLayerId = ReservedLayerId + int32(EDrawLayer::EventBorder);

			for (const FRenderGraphTrackDrawState::FBoxPrimitive& Box : DrawState.Borders)
			{
				const float Y = GetLaneY(TopLaneY, Layout, Box.DepthY);
				const float H = GetLaneHeight(Layout, Box.DepthH);
				DrawContext.DrawBox(EventBorderLayerId, Box.X, Y, Box.W, H, WhiteBrush, Box.Color);
			}
		}

		// Draw texts.
		if (Layout.EventH > 10.0f)
		{
			const int32 EventTextLayerId = ReservedLayerId + int32(EDrawLayer::EventText);
			for (const FRenderGraphTrackDrawState::FTextPrimitive& Text : DrawState.Texts)
			{
				const float Y = GetLaneY(TopLaneY, Layout, Text.Depth) + 1.0f;
				DrawContext.DrawText(EventTextLayerId, Text.X, Y, Text.Text, EventFont, Text.bWhite ? FLinearColor::White : FLinearColor::Black);
			}
		}

		{
			const EDrawLayer SplineLayer = EDrawLayer::Foreground;
			for (const FSplinePrimitive& Spline : DrawState.Splines)
			{
				DrawSpline(Track, Spline, SplineLayer);
			}
		}
	}
}

void FRenderGraphTrackDrawHelper::DrawFadedEvents(const FRenderGraphTrackDrawState& DrawState, const FBaseTimingTrack& Track, const float Opacity) const
{
	const float TrackH = Track.GetHeight();
	if (TrackH > 0.0f)
	{
		const FTimingViewLayout& Layout = Viewport.GetLayout();

		float TopLaneY = Track.GetPosY() + Layout.TimelineDY + 1.0f;

		// Draw filled boxes (merged borders).
		{
			const int32 EventFillLayerId = ReservedLayerId + int32(EDrawLayer::EventFill);

			for (const FRenderGraphTrackDrawState::FBoxPrimitive& Box : DrawState.Boxes)
			{
				const float Y = GetLaneY(TopLaneY, Layout, Box.DepthY);
				const float H = GetLaneHeight(Layout, Box.DepthH);
				DrawContext.DrawBox(EventFillLayerId, Box.X, Y, Box.W, H, WhiteBrush, Box.Color.CopyWithNewOpacity(Opacity));
			}
		}

		// Draw filled boxes (event inside area).
		if (Layout.EventH > 2.0f)
		{
			const int32 EventFillLayerId = ReservedLayerId + int32(EDrawLayer::EventFill);

			for (const FRenderGraphTrackDrawState::FBoxPrimitive& Box : DrawState.InsideBoxes)
			{
				const float Y = GetLaneY(TopLaneY, Layout, Box.DepthY) + 1.0f;
				const float H = GetLaneHeight(Layout, Box.DepthH) - 2.0f;
				DrawContext.DrawBox(EventFillLayerId, Box.X, Y, Box.W, H, WhiteBrush, Box.Color.CopyWithNewOpacity(Opacity));
			}
		}

		// Draw borders.
		{
			const int32 EventBorderLayerId = ReservedLayerId + int32(EDrawLayer::EventBorder);

			for (const FRenderGraphTrackDrawState::FBoxPrimitive& Box : DrawState.Borders)
			{
				const float Y = GetLaneY(TopLaneY, Layout, Box.DepthY);
				const float H = GetLaneHeight(Layout, Box.DepthH);
				FSlateRoundedBoxBrush Brush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 0.0f, Box.Color.CopyWithNewOpacity(Opacity), 1.0f);
				DrawContext.DrawBox(EventBorderLayerId, Box.X, Y, Box.W, H, &Brush, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
			}
		}

		// Draw texts.
		if (Layout.EventH > 10.0f)
		{
			const FLinearColor WhiteColor(1.0f, 1.0f, 1.0f, Opacity);
			const FLinearColor BlackColor(0.0f, 0.0f, 0.0f, Opacity);

			const int32 EventTextLayerId = ReservedLayerId + int32(EDrawLayer::EventText);
			for (const FRenderGraphTrackDrawState::FTextPrimitive& Text : DrawState.Texts)
			{
				const float Y = GetLaneY(TopLaneY, Layout, Text.Depth) + 1.0f;
				DrawContext.DrawText(EventTextLayerId, Text.X, Y, Text.Text, EventFont, Text.bWhite ? WhiteColor : BlackColor);
			}
		}
	}
}

void FRenderGraphTrackDrawHelper::DrawTimingEventHighlight(const FBaseTimingTrack& Track, double StartTime, double EndTime, float DepthY, float DepthH, EDrawEventMode Mode) const
{
	float EventX1 = Viewport.TimeToSlateUnitsRounded(StartTime);
	if (EventX1 > Viewport.GetWidth())
	{
		return;
	}

	EndTime = Viewport.RestrictEndTime(EndTime);
	float EventX2 = Viewport.TimeToSlateUnitsRounded(EndTime);
	if (EventX2 < 0)
	{
		return;
	}

	// Timing events are displayed with minimum 1px (including empty ones).
	if (EventX1 == EventX2)
	{
		EventX2 = EventX1 + 1.0f;
	}

	// Limit event width on the viewport's left side.
	const float MinX = -2.0f; // -2 allows event border to remain outside screen
	if (EventX1 < MinX)
	{
		EventX1 = MinX;
	}

	// Limit event width on the viewport's right side.
	const float MaxX = Viewport.GetWidth() + 2.0f; // +2 allows event border to remain outside screen
	if (EventX2 > MaxX)
	{
		EventX2 = MaxX;
	}

	const FTimingViewLayout& Layout = Viewport.GetLayout();

	const float EventW = EventX2 - EventX1;
	const float EventY = Track.GetPosY() + GetLaneY(Layout, DepthY);
	const float EventH = GetLaneHeight(Layout, DepthH);

	const int32 LayerId = ReservedLayerId + int32(EDrawLayer::EventHighlight);

	constexpr float BorderOffset = 2.0f;

	if (Mode == EDrawEventMode::Hovered)
	{
		const FLinearColor Color(1.0f, 1.0f, 0.0f, 1.0f); // yellow

		// Draw border around the timing event box.
		FSlateRoundedBoxBrush Brush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 2.0f, Color, 2.0f);
		DrawContext.DrawBox(LayerId, EventX1 - BorderOffset, EventY - BorderOffset, EventW + 2 * BorderOffset, EventH + 2 * BorderOffset, &Brush, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
	}
	else // EDrawEventMode::Selected or EDrawEventMode::SelectedAndHovered
	{
		// Animate color from white (if selected and hovered) or yellow (if only selected) to black, using a squared sine function.
		const double Time = static_cast<double>(FPlatformTime::Cycles64()) * FPlatformTime::GetSecondsPerCycle64();
		float S = FMath::Sin(2.0 * Time);
		S = S * S; // squared, to ensure only positive [0 - 1] values
		const float Blue = (Mode == EDrawEventMode::SelectedAndHovered) ? 0.0f : S;
		const FLinearColor Color(S, S, Blue, 1.0f);

		// Draw border around the timing event box.
		FSlateRoundedBoxBrush Brush(FLinearColor(0.0f, 0.0f, 0.0f, 0.0f), 2.0f, Color, 2.0f);
		DrawContext.DrawBox(LayerId, EventX1 - BorderOffset, EventY - BorderOffset, EventW + 2 * BorderOffset, EventH + 2 * BorderOffset, &Brush, FLinearColor(0.0f, 0.0f, 0.0f, 0.0f));
	}
}

void FRenderGraphTrackDrawHelper::DrawTrackHeader(const FBaseTimingTrack& Track) const
{
	const int32 HeaderLayerId = ReservedLayerId + int32(EDrawLayer::HeaderBackground);
	const int32 HeaderTextLayerId = ReservedLayerId + int32(EDrawLayer::HeaderText);

	DrawTrackHeader(Track, HeaderLayerId, HeaderTextLayerId);
}

void FRenderGraphTrackDrawHelper::DrawTrackHeader(const FBaseTimingTrack& Track, const int32 HeaderLayerId, const int32 HeaderTextLayerId) const
{
	const float TrackY = Track.GetPosY();
	const float TrackH = Track.GetHeight();

	if (TrackH > 0.0f)
	{
		// Draw a horizontal line between timelines (top line of a track).
		DrawContext.DrawBox(HeaderLayerId, 0.0f, TrackY, Viewport.GetWidth(), 1.0f, WhiteBrush, EdgeColor);

		// Draw header with name of timeline.
		if (TrackH > 4.0f)
		{
			const TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
			float TextWidth = FontMeasureService->Measure(Track.GetName(), EventFont).X;

			constexpr float PinWidth = 8.0f;
			if (Track.IsSelected())
			{
				TextWidth += PinWidth;
			}

			const float HeaderX = 0.0f;
			const float HeaderW = TextWidth + 4.0f;

			const float HeaderY = TrackY + 1.0f;
			const float HeaderH = FMath::Min(12.0f, Track.GetHeight() - 1.0f);

			if (HeaderH > 0)
			{
				DrawContext.DrawBox(HeaderLayerId, HeaderX, HeaderY, HeaderW, HeaderH, WhiteBrush, EdgeColor);

				const FLinearColor TextColor = GetTrackNameTextColor(Track);

				float TextX = HeaderX + 2.0f;
				const float TextY = HeaderY + HeaderH / 2.0f - 7.0f;

				if (Track.IsSelected())
				{
					DrawContext.DrawText(HeaderTextLayerId, TextX, TextY, TEXT("\u25CA"), EventFont, TextColor); // lozenge shape
					TextX += PinWidth;
				}

				DrawContext.DrawText(HeaderTextLayerId, TextX, TextY, Track.GetName(), EventFont, TextColor);
			}
		}
	}
}

void FRenderGraphTrackDrawHelper::DrawSpline(const FBaseTimingTrack& Track, const FSplinePrimitive& Spline, EDrawLayer Layer) const
{
	const float ViewportWidth = Viewport.GetWidth();

	const float Guardband = 1024.0f;
	float StartX = FMath::Max(Spline.Start.X               , -Guardband);
	float EndX   = FMath::Min(Spline.Start.X + Spline.End.X,  Guardband + ViewportWidth);
	const float Width = EndX - StartX;

	if (StartX > ViewportWidth || EndX < 0.0f)
	{
		return;
	}

	const FTimingViewLayout& Layout = Viewport.GetLayout();

	const float TopLaneY  = Track.GetPosY() + Layout.TimelineDY + 1.0f;
	const float StartY    = GetLaneY(TopLaneY, Layout, Spline.Start.Y);
	const float EndY      = GetChildLaneY(Layout, Spline.End.Y);

	DrawContext.DrawSpline(
		ReservedLayerId + int32(Layer),
		StartX,
		StartY,
		FVector2D::ZeroVector,
		Spline.StartDir,
		FVector2D(Width, EndY),
		Spline.EndDir,
		Spline.Thickness,
		Spline.Tint
	);
}

void FRenderGraphTrackDrawHelper::DrawBox(const FBaseTimingTrack& Track, float SlateX, float DepthY, float SlateW, float DepthH, FLinearColor Color, EDrawLayer Layer) const
{
	const float ViewportWidth = Viewport.GetWidth();

	const float Guardband = 1024.0f;
	const float SlateMinX = FMath::Max(SlateX         , -Guardband);
	const float SlateMaxX = FMath::Min(SlateX + SlateW,  Guardband + ViewportWidth);
	SlateW = SlateMaxX - SlateMinX;

	if (SlateMinX > ViewportWidth || SlateMaxX < 0.0f)
	{
		return;
	}

	const FTimingViewLayout& Layout = Viewport.GetLayout();

	const float SlateY = GetLaneY(Layout, DepthY) + Track.GetPosY();
	const float SlateH = GetLaneHeight(Layout, DepthH);

	DrawContext.DrawBox(ReservedLayerId + int32(Layer), SlateMinX, SlateY, SlateW, SlateH, WhiteBrush, Color);
}

FLinearColor FRenderGraphTrackDrawHelper::GetTrackNameTextColor(const FBaseTimingTrack& Track) const
{
	return  Track.IsHovered() ? FLinearColor(1.0f, 1.0f, 0.0f, 1.0f) :
		Track.IsSelected() ? FLinearColor(1.0f, 1.0f, 0.5f, 1.0f) :
		FLinearColor(1.0f, 1.0f, 1.0f, 1.0f);
}

}} // namespace UE::RenderGraphInsights
