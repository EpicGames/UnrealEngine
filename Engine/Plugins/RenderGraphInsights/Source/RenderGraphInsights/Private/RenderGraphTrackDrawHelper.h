// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Fonts/SlateFontInfo.h"

#include "Insights/ViewModels/TimingViewLayout.h"

class ITimingViewDrawHelper;

struct FSlateBrush;
struct FDrawContext;

class ITimingTrackDrawContext;
class FTimingTrackViewport;
class FBaseTimingTrack;

enum class EDrawEventMode : uint32;

namespace UE { namespace RenderGraphInsights {
	
inline float GetChildLaneY(const FTimingViewLayout& Layout, float DepthY)
{
	return (Layout.EventH + Layout.EventDY) * DepthY;
}

inline float GetLaneY(float TopLaneY, const FTimingViewLayout& Layout, float DepthY)
{
	return TopLaneY + GetChildLaneY(Layout, DepthY);
}

inline float GetLaneY(const FTimingViewLayout& Layout, float DepthY)
{
	return GetLaneY(Layout.TimelineDY + 1.0f, Layout, DepthY);
}

inline float GetLaneHeight(const FTimingViewLayout& Layout, float DepthH)
{
	return Layout.EventH * DepthH + Layout.EventDY * (FMath::CeilToFloat(DepthH) - 1);
}

inline float GetDepthFromLaneY(const FTimingViewLayout& Layout, float Y)
{
	return (Y - 1.0f - Layout.TimelineDY) / (Layout.EventH + Layout.EventDY);
}

class FRenderGraphBaseTrack;

enum class EDrawLayer : int32
{
	Background,
	EventBorder,
	EventFill,
	EventText,
	EventHighlight,
	Relation,
	HeaderBackground,
	HeaderText,
	Foreground,
	Count,
};

struct FSplinePrimitive
{
	float Thickness = 1.0f;
	FVector2D Start = FVector2D::ZeroVector;
	FVector2D StartDir = FVector2D::ZeroVector;
	FVector2D End = FVector2D::ZeroVector;
	FVector2D EndDir = FVector2D::ZeroVector;
	FLinearColor Tint = FLinearColor::White;
};

struct FRenderGraphTrackDrawState
{
	struct FBoxPrimitive
	{
		float DepthY{};
		float DepthH{};
		float X{};
		float W{};
		FLinearColor Color = FLinearColor::White;
	};

	struct FTextPrimitive
	{
		float Depth{};
		float X{};
		FString Text{};
		bool bWhite{};
		FLinearColor Color = FLinearColor::White;
	};

	void Reset()
	{
		Boxes.Reset();
		InsideBoxes.Reset();
		Borders.Reset();
		Texts.Reset();
		Splines.Reset();
	}

	TArray<FBoxPrimitive> Boxes;
	TArray<FBoxPrimitive> InsideBoxes;
	TArray<FBoxPrimitive> Borders;
	TArray<FTextPrimitive> Texts;
	TArray<FSplinePrimitive> Splines;
};

class FRenderGraphTrackDrawStateBuilder
{
private:
	struct FBoxData
	{
		float X1{};
		float X2{};
		uint32 Color{};
		FLinearColor LinearColor;

		void Reset() { X1 = 0.0f; X2 = 0.0f; Color = 0; }
	};

public:
	explicit FRenderGraphTrackDrawStateBuilder(FRenderGraphTrackDrawState& InState, const FTimingTrackViewport& InViewport);

	FRenderGraphTrackDrawStateBuilder(const FRenderGraphTrackDrawStateBuilder&) = delete;
	FRenderGraphTrackDrawStateBuilder& operator=(const FRenderGraphTrackDrawStateBuilder&) = delete;

	const FTimingTrackViewport& GetViewport() const { return Viewport; }

	using GetEventNameCallback = TFunctionRef<const FString(float /*AvailableWidth*/)>;

	void AddEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, const TCHAR* InEventName, uint64 InEventType = 0, uint32 InEventColor = 0);
	void AddEvent(double InEventStartTime, double InEventEndTime, uint32 InEventDepth, uint32 InEventColor, GetEventNameCallback InGetEventNameCallback);
	void AddEvent(double InEventStartTime, double InEventEndTime, float InEventDepthY, float InEventDepthH, uint32 InEventColor, GetEventNameCallback InGetEventNameCallback);
	void Flush();

	void AddSpline(const FSplinePrimitive& Spline)
	{
		DrawState.Splines.Add(Spline);
	}

	int32 GetMaxDepth() const { return FMath::Max(MaxDepth, MaxDepthCached); }

private:
	void FlushBox(const FBoxData& Box, const int32 Depth);

	static void AppendDurationToEventName(FString& InOutEventName, const double InDuration);

	FRenderGraphTrackDrawState& DrawState; // cached draw state to build
	const FTimingTrackViewport& Viewport;

	int32 MaxDepth = -1;
	int32 MaxDepthCached = -1;

	TArray<float> LastEventX2; // X2 value for last event on each depth
	TArray<FBoxData> LastBox;

	const FSlateFontInfo EventFont;
};

class FRenderGraphTrackDrawHelper final
{
public:
	explicit FRenderGraphTrackDrawHelper(const ITimingTrackDrawContext& Context);

	FRenderGraphTrackDrawHelper(const FRenderGraphTrackDrawHelper&) = delete;
	FRenderGraphTrackDrawHelper& operator=(const FRenderGraphTrackDrawHelper&) = delete;

	const FDrawContext& GetDrawContext() const { return DrawContext; }
	const FTimingTrackViewport& GetViewport() const { return Viewport; }

	void DrawEvents(const FRenderGraphTrackDrawState& DrawState, const FBaseTimingTrack& Track) const;
	void DrawFadedEvents(const FRenderGraphTrackDrawState& DrawState, const FBaseTimingTrack& Track, const float Opacity = 0.1f) const;

	void DrawTimingEventHighlight(const FBaseTimingTrack& Track, double StartTime, double EndTime, float DepthY, float DepthH, EDrawEventMode Mode) const;

	void DrawTrackHeader(const FBaseTimingTrack& Track) const;
	void DrawTrackHeader(const FBaseTimingTrack& Track, const int32 HeaderLayerId, const int32 HeaderTextLayerId) const;

	void DrawSpline(const FBaseTimingTrack& Track, const FSplinePrimitive& Spline, EDrawLayer Layer) const;

	void DrawBox(const FBaseTimingTrack& Track, float SlateX, float DepthY, float SlateW, float DepthH, FLinearColor Color, EDrawLayer Layer) const;

	FLinearColor GetTrackNameTextColor(const FBaseTimingTrack& Track) const;

	FLinearColor GetEdgeColor() const { return EdgeColor; }
	const FSlateFontInfo& GetEventFont() const { return EventFont; }
	const FSlateBrush* GetWhiteBrush() const { return WhiteBrush; }
	int32 GetFirstLayerId() const { return ReservedLayerId; }
	int32 GetNumLayerIds() const { return int32(EDrawLayer::Count); }

private:

	const FDrawContext& DrawContext;
	const FTimingTrackViewport& Viewport;
	const ITimingViewDrawHelper& ParentHelper;

	const FSlateBrush* WhiteBrush;
	const FSlateBrush* EventBorderBrush;
	const FSlateBrush* HoveredEventBorderBrush;
	const FSlateBrush* SelectedEventBorderBrush;
	const FLinearColor ValidAreaColor;
	const FLinearColor InvalidAreaColor;
	const FLinearColor EdgeColor;
	const FSlateFontInfo EventFont;
	const int32 ReservedLayerId{};
};

}} // UE::RenderGraphInsights
